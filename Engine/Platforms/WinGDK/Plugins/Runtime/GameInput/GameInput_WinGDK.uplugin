{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [ "WinGDK" ],
	"Modules": [
		{
			"Name": "GameInputBase",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "WinGDK" ]
		}
	]
}