// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameInputBase_WinGDK : GameInputBase
	{
		// Right now, the current GDK version does not support "arm64", only "Win64" and "arm64ec" (ec = emulation compatible)
		protected override bool HasGameInputSupport => 
			Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) &&
			Target.Architecture != UnrealArch.Arm64;

		public GameInputBase_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{

		}
	}
}