{
	"FileVersion": 3,
	"FriendlyName": "Game Input Base",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "GameInput is a next-generation input API that exposes input devices of all kinds through a single consistent interface.",
	"Category": "Input Devices",
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"CanContainContent": false,
	"RequiresBuildPlatform": false,
	"HasExplicitPlatforms": true,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "GameInputBase",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "GameInputBaseEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}