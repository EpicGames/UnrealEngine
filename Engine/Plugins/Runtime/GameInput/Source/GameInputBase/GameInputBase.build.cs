// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class GameInputBase : ModuleRules
	{
		/**
		 * Conditions which are required for this build target to have support for the GameInput library.
		 * Temporarily disabled for WinArm64 becasue the GameInput lib is not available for arm64 yet.  This is expected to change by second quater 2026.
		 */
		protected virtual bool HasGameInputSupport => Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.Architecture != UnrealArch.Arm64;

		/**
		* True on platforms which support the "Dynamic Latency Input" API of Game Input.
		*/
		protected virtual bool bSupportsDynamicLatencyInput => false;

		/**
		 * True on platforms which support Haptics through game input.
		 */
		protected virtual bool bSupportsHapticDevices => true;

		/**
		 * Array of required module names to be added to PublicDependencyModuleNames
		 * if the target supports game input
		 */
		protected virtual string[] RequiredModuleDepNames => ["GameInputWindowsLibrary"];

		public GameInputBase(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable truncation warnings in this plugin
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			// Uncomment this line to make for easier debugging
			//OptimizeCode = CodeOptimization.Never;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"ApplicationCore",
					"SlateCore",
					"Slate",
					"Engine",
					"InputCore",
					"InputDevice",				
					"CoreUObject",
					"DeveloperSettings",
					// AudioExtensions is always a public dep because UGameInputHapticEndpointSettings (UCLASS)
					// must not be inside preprocessor blocks per UHT requirements.
					"AudioExtensions",
				}
			);
			
			// Define this as 0 in the base module to avoid compilation errors when building
			// without any Game Input support. It is up to the platform-specific submodules to define 
			PublicDefinitions.Add("GAME_INPUT_SUPPORT=" + (HasGameInputSupport ? "1" : "0"));			

			PublicDefinitions.Add("UE_GAME_INPUT_SUPPORTS_DLI=" + (HasGameInputSupport && bSupportsDynamicLatencyInput ? "1" : "0"));
			PublicDefinitions.Add("UE_GAME_INPUT_SUPPORTS_HAPTICS=" + (HasGameInputSupport && bSupportsHapticDevices ? "1" : "0"));
			
			// Give platforms extensions a chance to add any required dependencies that may be necessary to compile game input
			if (HasGameInputSupport)
			{
				PublicDependencyModuleNames.AddRange(RequiredModuleDepNames);
			}

			// WASAPI and audio mixer dependencies for haptic audio support
			if (bSupportsHapticDevices && HasGameInputSupport)
			{
				PrivateDependencyModuleNames.AddRange(new string[] { "AudioMixerCore", "AudioMixer" });

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
				{
					// FScopedCoInitialize lives here; also provides a shared MM device enumeration surface.
					PrivateDependencyModuleNames.Add("WindowsMMDeviceEnumeration");

					// WASAPI / MMDevice API linkage. combaseapi (CoCreateInstance, CoTaskMemFree) and
					// audioclient come from the Windows SDK implicit libs, but mmdevapi.lib exposes
					// MMDeviceEnumerator's CLSID and IMMDevice vtable — link it explicitly.
					PublicSystemLibraries.AddRange(new string[] { "mmdevapi.lib", "Avrt.lib" });
				}
			}
		}
	}
}
