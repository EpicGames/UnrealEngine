// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputBaseModule.h"

#include <atomic>
#include "CoreGlobals.h"
#include "GameInputKeyTypes.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Tasks/Task.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
#include "GameInputHapticEndpointFactory.h"
#include "GameInputDeveloperSettings.h"
#include "IAudioEndpoint.h"
#endif	// #if UE_GAME_INPUT_SUPPORTS_HAPTICS

#define LOCTEXT_NAMESPACE "GameInputBaseModule" 

namespace UE::GameInput
{
#if GAME_INPUT_SUPPORT
	// A singleton pointer to the base GameInput interface. 
	// This provides access to reading the input stream, device callbacks, and more. 
	static TComPtr<IGameInput> GGameInputInterface;
#endif	// #if GAME_INPUT_SUPPORT	
}

#if PLATFORM_WINDOWS && GAME_INPUT_SUPPORT
	FCriticalSection FGameInputBaseModule::GameInputCreationLock;
#endif

FGameInputBaseModule& FGameInputBaseModule::Get()
{
	return FModuleManager::LoadModuleChecked<FGameInputBaseModule>(TEXT("GameInputBase"));
}

bool FGameInputBaseModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(TEXT("GameInputBase"));
}

void FGameInputBaseModule::StartupModule()
{
	UE_LOGF(LogGameInput, Log, "GameInputBase module startup...");
	
	// We don't care for Game Input if we are running a commandlet, like when we are cooking.
	if (IsRunningCommandlet())
	{
		UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because IsRunningCommandlet is true. GameInput will not be initialized.");
		return;
	}

	// If there is no project name then we don't need game input either. This means we are in the project launcher
	if (!FApp::HasProjectName())
	{
		UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because there is no project name. GameInput will not be initialized.");
		return;
	}

	// Unattended app can't receive any user input, so there is no need to try and create the GameInput interface.
	if (FApp::IsUnattended() && !FApp::AllowUnattendedInput())
	{
		UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because it is unattended (FApp::IsUnattended is true) and thus cannot receive user input. GameInput will not be initialized.");
		return;
	}

	// Doesn't make sense to have headless apps create game input
	if (!FApp::CanEverRender())
	{
		UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because it cannot render anything (FApp::CanEverRender is false). GameInput will not be initialized.");
		return;
	}

#if GAME_INPUT_SUPPORT

	UE_LOGF(LogGameInput, Log, "Compiled using GameInput major Version: %d", GAMEINPUT_API_VERSION);
	
#if PLATFORM_WINDOWS

	if (!UE::GameInput::HasValidVersionOfGameInput())
	{
		UE_LOGF(LogGameInput, Warning, "[%hs] module is exiting because a valid GameInput version is not installed. GameInput will not be initialized. Is it installed correctly?", __func__);
		return;
	}

	UE_LOGF(LogGameInput, Log, "Executing with detected GameInput Version: %ls", *UE::GameInput::GetInstalledGameInputVersionInfoString());

	// The GameInputCreate function can be expensive on windows (~5s on startup!) so run it in an async task
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [FinishedDelegate = MakeShared<TMulticastDelegate<void(IGameInput*)>>(OnGameInputCreation)]()
	{		
		GameInputCreationLock.Lock();

		const HRESULT HResult = GameInputCreate(&UE::GameInput::GGameInputInterface);
		
		GameInputCreationLock.Unlock();
		
		if (SUCCEEDED(HResult))
		{
			UE_LOGF(LogGameInput, Log, "[FGameInputBaseModule::StartupModule] Successfully created the IGameInput interface");
		}
		else
		{
			UE_LOGF(LogGameInput, Warning, "Failed to initialize GameInput: 0x%X", HResult);	
		}

		struct FBroadcastCreationDelegateFunctor
		{
			TSharedPtr<TMulticastDelegate<void(IGameInput*)>> DelegatePtr = nullptr;
			
			explicit FBroadcastCreationDelegateFunctor(TSharedPtr<TMulticastDelegate<void(IGameInput*)>> InDelegatePtr)
				: DelegatePtr(InDelegatePtr)
			{}
			
			void operator()()
			{
				if (DelegatePtr)
				{
					DelegatePtr->Broadcast(UE::GameInput::GGameInputInterface);
				}
			}
		};
		
		// Broadcast the creation delegate on the game thread, because IInputDevice's run only on the game thread.
		// This module start up is running on EnginePreinit. This object is not needed until the actual engine tick
		// when we create our IInputDevices on the platform (IInputDevice::CreateInputDevice).
		// Since we are within an async background task, we cannot gaurantee that this will actually finish
		// by the time the first engine tick occurs. For that reason, we need to broadcast this creation
		// delegate so that the IInputDevice can listen for it an handle it accordingly.
		UE::Tasks::Launch(
			UE_SOURCE_LOCATION,
			FBroadcastCreationDelegateFunctor(FinishedDelegate),
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
	});

#else 	// PLATFORM_WINDOWS
	
	// Create the Game Input interface
	const HRESULT HResult = GameInputCreate(&UE::GameInput::GGameInputInterface);
	if (SUCCEEDED(HResult))
	{
		UE_LOGF(LogGameInput, Log, "[FGameInputBaseModule::StartupModule] Successfully created the IGameInput interface");
	}
	else
	{
		UE_LOGF(LogGameInput, Error, "Failed to initialize GameInput: 0x%X", HResult);
	}
	OnGameInputCreation.Broadcast(UE::GameInput::GGameInputInterface);
	
#endif	// !PLATFORM_WINDOWS

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	InitializeHapticAudioFactory();
#endif
	
	InitializeGameInputKeys();
	
#else
	UE_LOGF(LogGameInput, Warning, "Failed to initialize GameInput! GAME_INPUT_SUPPORT is false!");
#endif	// #if GAME_INPUT_SUPPORT
}

void FGameInputBaseModule::ShutdownModule()
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	ShutdownHapticAudioFactory();
#endif

#if GAME_INPUT_SUPPORT

#if PLATFORM_WINDOWS
	FScopeLock Lock(&GameInputCreationLock);
#endif
	
	UE::GameInput::GGameInputInterface.Reset();	

#endif // #if GAME_INPUT_SUPPORT
}

#if GAME_INPUT_SUPPORT
IGameInput* FGameInputBaseModule::GetGameInput()
{
#if PLATFORM_WINDOWS
	FScopeLock ScopeLock(&GameInputCreationLock);
#endif
	
	return UE::GameInput::GGameInputInterface;
}
#endif	// GAME_INPUT_SUPPORT


void FGameInputBaseModule::InitializeGameInputKeys()
{
#if GAME_INPUT_SUPPORT

	static const FName MenuCategory = TEXT("GameInput");
	EKeys::AddMenuCategoryDisplayInfo(MenuCategory, LOCTEXT("GameInput", "Game Input"), TEXT("GraphEditor.PadEvent_16x"));

	//  
	// Racing Wheel
	//

	// Analog types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Brake, LOCTEXT("GameInput_RacingWheel_Brake", "Game Input Racing Wheel Brake"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Clutch, LOCTEXT("GameInput_RacingWheel_Clutch", "Game Input Racing Wheel Clutch"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Handbrake, LOCTEXT("GameInput_RacingWheel_Handbrake", "Game Input Racing Wheel Handbrake"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Throttle, LOCTEXT("GameInput_RacingWheel_Throttle", "Game Input Racing Wheel Throttle"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Wheel, LOCTEXT("GameInput_RacingWheel_Wheel", "Game Input Racing Wheel"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_PatternShifterGear, LOCTEXT("GameInput_RacingWheel_PatternShifterGear", "Game Input Racing Wheel Pattern Shifter Gear"), FKeyDetails::Axis1D, MenuCategory));

	// Button types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_None, LOCTEXT("GameInput_RacingWheel_None", "Game Input Racing Wheel None"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_Menu, LOCTEXT("GameInput_RacingWheel_Menu", "Game Input Racing Wheel Menu"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_View, LOCTEXT("GameInput_RacingWheel_View", "Game Input Racing Wheel View"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_PreviousGear, LOCTEXT("GameInput_RacingWheel_PreviousGear", "Game Input Racing Wheel Previous Gear"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::RacingWheel_NextGear, LOCTEXT("GameInput_RacingWheel_NextGear", "Game Input Racing Wheel Next Gear"), FKeyDetails::GamepadKey, MenuCategory));

	//
	// Flight Stick
	//

	// Analog types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_Roll, LOCTEXT("GameInput_FlightStick_Roll", "Game Input Flight Stick Roll"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_Pitch, LOCTEXT("GameInput_FlightStick_Pitch", "Game Input Flight Stick Pitch"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_Yaw, LOCTEXT("GameInput_FlightStick_Yaw", "Game Input Flight Stick Yaw"), FKeyDetails::Axis1D, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_Throttle, LOCTEXT("GameInput_FlightStick_Throttle", "Game Input Flight Stick Throttle"), FKeyDetails::Axis1D, MenuCategory));

	// Button types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_None, LOCTEXT("GameInput_FlightStick_None", "Game Input Flight Stick None"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_Menu, LOCTEXT("GameInput_FlightStick_Menu", "Game Input Flight Stick Menu"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_View, LOCTEXT("GameInput_FlightStick_View", "Game Input Flight Stick View"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_FirePrimary, LOCTEXT("GameInput_FlightStick_FirePrimary", "Game Input Flight Stick Fire Primary"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::FlightStick_FireSecondary, LOCTEXT("GameInput_FlightStick_FireSecondary", "Game Input Flight Stick Fire Secondary"), FKeyDetails::GamepadKey, MenuCategory));

	//
	// Arcade Stick    
	//

	// Button Types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action1, LOCTEXT("GameInput_ArcadeStick_Action1", "Game Input Arcade Stick Action 1"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action2, LOCTEXT("GameInput_ArcadeStick_Action2", "Game Input Arcade Stick Action 2"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action3, LOCTEXT("GameInput_ArcadeStick_Action3", "Game Input Arcade Stick Action 3"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action4, LOCTEXT("GameInput_ArcadeStick_Action4", "Game Input Arcade Stick Action 4"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action5, LOCTEXT("GameInput_ArcadeStick_Action5", "Game Input Arcade Stick Action 5"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Action6, LOCTEXT("GameInput_ArcadeStick_Action6", "Game Input Arcade Stick Action 6"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Special1, LOCTEXT("GameInput_ArcadeStick_Special1", "Game Input Arcade Stick Special 1"), FKeyDetails::GamepadKey, MenuCategory));
	EKeys::AddKey(FKeyDetails(FGameInputKeys::ArcadeStick_Special2, LOCTEXT("GameInput_ArcadeStick_Special2", "Game Input Arcade Stick Special 2"), FKeyDetails::GamepadKey, MenuCategory));
	
	//
	// Sensor Types
	//
	
	// Analog types
	EKeys::AddKey(FKeyDetails(FGameInputKeys::Sensor_HeadingInDegreesFromMagneticNorth, LOCTEXT("GameInput_Sensor_HeadingInDegreesFromMagneticNorth", "Game Input Heading From Magnetic North (In Degrees)"), FKeyDetails::Axis1D, MenuCategory));

#endif	// GAME_INPUT_SUPPORT
}

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
void FGameInputBaseModule::InitializeHapticAudioFactory()
{
	// Create the haptic endpoint factory early — before the audio device initializes
	// and calls InitializeSoundSubmixes. If we create it later (e.g. in IGameInputDeviceInterface
	// ctor, which runs on first input tick), the endpoint submix will already have been
	// initialized with a dummy factory and our CreateNewEndpointInstance will never be called.
	if (GetDefault<UGameInputDeveloperSettings>()->IsHapticSupportEnabled()
		&& FGenericPlatformMisc::IsPreferredInputDevice(TEXT("GameInput")))
	{
		UE_LOG(LogGameInput, Log, TEXT("GameInputBase: creating haptic endpoint factory (Vibration Output)"));
		
		HapticAudioFactory = MakeUnique<FGameInputHapticEndpointFactory>();
	}
}

void FGameInputBaseModule::ShutdownHapticAudioFactory()
{
	HapticAudioFactory.Reset();
}
#endif

IMPLEMENT_MODULE(FGameInputBaseModule, GameInputBase)

#undef LOCTEXT_NAMESPACE