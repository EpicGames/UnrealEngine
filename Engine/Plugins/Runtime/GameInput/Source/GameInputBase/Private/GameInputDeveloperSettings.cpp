// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputDeveloperSettings.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "GameInputLogging.h"
#include "GameInputBaseIncludes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameInputDeveloperSettings)

FGameInputDeviceIdentifier::FGameInputDeviceIdentifier()
	: VendorId(0)
	, ProductId(0)
{
}

FGameInputDeviceIdentifier::FGameInputDeviceIdentifier(uint16 InVendorId, uint16 InProductId)
	: VendorId(InVendorId)
	, ProductId(InProductId)
{
}

uint32 GetTypeHash(const FGameInputDeviceIdentifier& InId)
{
	uint32 Hash = 0;
	Hash = HashCombine(Hash, GetTypeHash(InId.VendorId));
	Hash = HashCombine(Hash, GetTypeHash(InId.ProductId));
	return Hash;
}

FString FGameInputDeviceIdentifier::ToString() const
{
	return FString::Format(TEXT("VendorId: {0} ProductId {1}"), { VendorId, ProductId });
}

bool FGameInputDeviceIdentifier::operator==(const FGameInputDeviceIdentifier& Other) const
{
	return 
		VendorId == Other.VendorId && 
		ProductId == Other.ProductId;
}

bool FGameInputDeviceIdentifier::operator!=(const FGameInputDeviceIdentifier& Other) const
{
	return !FGameInputDeviceIdentifier::operator==(Other);
}

FGameInputDeviceConfiguration::FGameInputDeviceConfiguration()
	: DeviceIdentifier(0, 0)
	, bOverrideHardwareDeviceIdString(false)
	, OverriddenHardwareDeviceId(TEXT(""))
	, bProcessControllerButtons(true)
	, bProcessControllerSwitchState(true)
	, bProcessControllerAxis(true)
	, bProcessRawReportData(false)
	, RawReportReadingId(0)
{
}

#if GAME_INPUT_SUPPORT
const FGameInputDeviceConfiguration* UGameInputDeveloperSettings::FindDeviceConfiguration(const GameInputDeviceInfo* const Info) const
{
	if (!Info)
	{
		return nullptr;
	}

	return DeviceConfigurations.FindByPredicate([&Info](const FGameInputDeviceConfiguration& DeviceConfig)
	{
		// Look for a matching Vendor ID and Product ID
		return
			(DeviceConfig.DeviceIdentifier.VendorId == Info->vendorId &&
			DeviceConfig.DeviceIdentifier.ProductId == Info->productId);
	});
}

const FGameInputDeviceConfiguration* UGameInputDeveloperSettings::FindDeviceConfiguration(const FGameInputDeviceIdentifier& HardwareID) const
{
	return DeviceConfigurations.FindByPredicate([&HardwareID](const FGameInputDeviceConfiguration& DeviceConfig)
	{
		// Look for a matching Vendor ID and Product ID
		return
			(DeviceConfig.DeviceIdentifier.VendorId == HardwareID.VendorId &&
			DeviceConfig.DeviceIdentifier.ProductId == HardwareID.ProductId);
	});
}

namespace UE::GameInput
{
	/** Converts the "raw" game input device family to the UE friendly property version. */
	static EGameInputDeviceFamily ConvertDeviceFamily(const GameInputDeviceFamily RawFamily)
	{
		switch (RawFamily)
		{
		case GameInputFamilyVirtual:
			return EGameInputDeviceFamily::Virtual;
#if GAMEINPUT_API_VERSION >= 3		
		case GameInputFamilyUnknown:
			return EGameInputDeviceFamily::Unknown;
#endif			
		case GameInputFamilyXboxOne:
			return EGameInputDeviceFamily::XboxOne;
		case GameInputFamilyXbox360:
			return EGameInputDeviceFamily::Xbox360;
		case GameInputFamilyHid:
			return EGameInputDeviceFamily::Hid;
		case GameInputFamilyI8042:
			return EGameInputDeviceFamily::I8042;
		case GameInputFamilyAggregate:
			return EGameInputDeviceFamily::Aggregate;
		}
		
		return EGameInputDeviceFamily::None;
	}
}

bool UGameInputDeveloperSettings::ShouldIgnoreDevice(const GameInputDeviceInfo* const Info) const
{
	// ignore any null devices
	if (!Info)
	{
		return true;
	}

	// We can ignore based on VID/PID...
	const FGameInputDeviceIdentifier DeviceId(Info->vendorId, Info->productId);
	if (DevicesToIgnore.Contains(DeviceId))
	{
		return true;
	}

	// We can also ignore based on the broader device family if desired
	const EGameInputDeviceFamily DeviceFamily = UE::GameInput::ConvertDeviceFamily(Info->deviceFamily);
	if (!EnumHasAllFlags(static_cast<EGameInputDeviceFamily>(SupportedDeviceFamilies), DeviceFamily))
	{
		return true;
	}

	// We fully support this device
	return false;
}

#endif	// GAME_INPUT_SUPPORT

UGameInputPlatformSettings* UGameInputPlatformSettings::Get()
{
	return UPlatformSettingsManager::Get().GetSettingsForPlatform<UGameInputPlatformSettings>();
}

void UGameInputPlatformSettings::InitializePlatformDefaults()
{
	// For platforms who are defaulted to KBM (Desktops) we want to disable KBM processing by default because
	// it will already be handled by the SlateApplication and platform message pump
	const FName PlatformName = GetPlatformIniName();
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);
	if (PlatformInfo.DefaultInputType == "MouseAndKeyboard")
	{
		bProcessKeyboard = false;
		bProcessMouse = false;

		// We also want to disable gamepad processing by default here, because it is likely that your PC platform
		// will be using the existing Xinput implementation. If one day we can turn off XInput by default,
		// then we can change this to be default on
		bProcessGamepad = false;
		bProcessController = false;
		bProcessRawInput = false;
	}

	// Everything else can just use the default values already set until we want to change something else
}

UGameInputDeveloperSettings::UGameInputDeveloperSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, SupportedDeviceFamilies(static_cast<uint32>(EGameInputDeviceFamily::All))
	, bDoNotProcessDuplicateCapabilitiesForSingleUser(true)
	, bEnableHapticSensorSupport(false)
{
	PlatformSpecificSettings.Initialize(UGameInputPlatformSettings::StaticClass());
}

bool UGameInputDeveloperSettings::IsHapticSupportEnabled() const
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	return bEnableHapticSensorSupport;
#else
	return false;
#endif
}

#if WITH_EDITOR
const TArray<uint32>& UGameInputDeveloperSettings::GetControllerButtonMappingDataKeyOptions()
{
	// The controller button bitmask is generated by bit shifting the current index
	// so just display all the possible options here
	static const TArray<uint32> Options =
	{
		1 << 0,
		1 << 1,
		1 << 2,
		1 << 3,
		1 << 4,
		1 << 5,
		1 << 6,
		1 << 7,
		1 << 8,
		1 << 9,
		1 << 10,
		1 << 11,
		1 << 12,
		1 << 13,
		1 << 14,
		1 << 15,
		1 << 16,
		1 << 17,
		1 << 18,
		1 << 19,
		1 << 21,
		1 << 22,
		1 << 23,
		1 << 24,
		1 << 25,
		1 << 26,
		1 << 27,
		1 << 28,
		1 << 29,
		1 << 30,
	};

	return Options;
}

void UGameInputDeveloperSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);	

	if (OnInputSettingChanged.IsBound())
	{
		OnInputSettingChanged.Broadcast();
	}
}
#endif	// WITH_EDITOR
