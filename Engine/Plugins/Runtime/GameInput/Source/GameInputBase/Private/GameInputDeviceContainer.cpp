// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputDeviceContainer.h"
#include "Framework/Application/SlateApplication.h"	// for GetPlatformCursor
#include "GenericPlatform/ICursor.h"
#include "GameInputUtils.h"
#include "GameInputLogging.h"
#include "GameInputDeveloperSettings.h"
#include "HAL/IConsoleManager.h"
#include "Processors/IGameInputDeviceProcessor.h"
#include "Processors/GameInputDeviceProcessor_Touch.h"
#include "Processors/GameInputDeviceProcessor_Sensor.h"
#include "Processors/GameInputDeviceProcessor_Raw.h"
#include "Processors/GameInputDeviceProcessor_RacingWheel.h"
#include "Processors/GameInputDeviceProcessor_KeyboardMouse.h"
#include "Processors/GameInputDeviceProcessor_Gamepad.h"
#include "Processors/GameInputDeviceProcessor_FlightStick.h"
#include "Processors/GameInputDeviceProcessor_Controller.h"
#include "Processors/GameInputDeviceProcessor_ArcadeStick.h"

#if GAME_INPUT_SUPPORT

FGameInputDeviceContainer::FGameInputDeviceContainer(
	const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler,
	IGameInputDevice* InDevice,
	GameInputKind InAllowedGameInputKinds,
	FPlatformUserId InUserId, 
	FInputDeviceId InDeviceId)
	: MessageHandler(InMessageHandler)
	, Device(InDevice)
	, AllowedGameInputKinds(InAllowedGameInputKinds)
	, UserId(InUserId)
	, AssignedDeviceId(InDeviceId)
	, IgnoreReadingTimestamp(0)
{
	if (!InDevice)
	{
		ensureAlwaysMsgf(false, TEXT("A Game Input container was created without a valid IGameInputDevice! This container will fail to process any input and we should not have gotten here."));
		return;
	}

	// Initalize the App Local ID, which should never change on this container
	if (const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device))
	{
		LocalDeviceId = Info->deviceId;
	}
}

void FGameInputDeviceContainer::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

void FGameInputDeviceContainer::InitalizeDeviceProcessors()
{
	if (!Device)
	{
		return;
	}

	// Based on the GameInputKind of this device, create any processors that are supported
	const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);

	const GameInputKind DeviceKind = Info->supportedInput;

	UE_LOGF(LogGameInput, Log, "InitalizeDeviceProcessors for device kind %ls with an allowed kind of %ls", *UE::GameInput::LexToString(DeviceKind), *UE::GameInput::LexToString(AllowedGameInputKinds));

	InitalizeDeviceProcessors_Impl();
}

void FGameInputDeviceContainer::RecreateDeviceProcessors(const GameInputKind InAllowedGameInputKinds)
{
	UE_CLOGF(AllowedGameInputKinds != InAllowedGameInputKinds,
		LogGameInput, Log, "[%s] Recreating device processors with new allowance. Changing from %ls to %ls",
		__func__, *UE::GameInput::LexToString(AllowedGameInputKinds), *UE::GameInput::LexToString(InAllowedGameInputKinds));
	
	AllowedGameInputKinds = InAllowedGameInputKinds;

	// Tear down the old input processors
	ClearInputState(nullptr);
	Processors.Empty();

	// Create them again with the new allowed input kinds
	InitalizeDeviceProcessors();
}

void FGameInputDeviceContainer::InitalizeDeviceProcessors_Impl()
{
	if (!Device)
	{
		return;
	}

	// Based on the GameInputKind of this device, create any processors that are supported
	const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);

	const GameInputKind DeviceKind = Info->supportedInput;

	if (DeviceKind & AllowedGameInputKinds & GameInputKindGamepad)
	{
		Processors.Emplace(MakeShared<FGameInputGamepadDeviceProcessor>());
	}
	
	if (DeviceKind & AllowedGameInputKinds & GameInputKindKeyboard)
	{
		Processors.Emplace(MakeShared<FGameInputKeyboardDeviceProcessor>());
	}
	
	// TODO: This was throwing a linker error on some platforms, investigate.
	//if (DeviceKind & AllowedGameInputKinds & GameInputKindMouse)
	//{
	//	// TODO: Make high precision mouse a config setting
	//	Processors.Emplace(MakeShared<FGameInputMouseDeviceProcessor>(FSlateApplication::Get().GetPlatformCursor()));
	//}

#if UE_GAMEINPUT_SUPPORTS_TOUCH
	if (DeviceKind & AllowedGameInputKinds & GameInputKindTouch)
	{
		Processors.Emplace(MakeShared<FGameInputTouchDeviceProcessor>());
	}
#endif	// UE_GAMEINPUT_SUPPORTS_TOUCH

	// Only allow for external controller devices have have specified configs if that is enabled
	const UGameInputDeveloperSettings* Settings = GetDefault<UGameInputDeveloperSettings>();
	const UGameInputPlatformSettings* PlatformSettings = UGameInputPlatformSettings::Get();
	
	// "Controller" types
	if (DeviceKind & AllowedGameInputKinds & GameInputKindController)
	{
		const bool bIsDeviceAllowed =
			PlatformSettings->bSpecialDevicesRequireExplicitDeviceConfiguration ?
			(Settings->FindDeviceConfiguration(Info) != nullptr) :
			true;

		if (bIsDeviceAllowed)
		{
			Processors.Emplace(MakeShared<FGameInputControllerDeviceProcessor>());
		}
		else
		{
			UE_LOGF(
				LogGameInput,
				Warning,
				"A game input controller device (%ls) was connected but will not be processed because it does not have an explict device configuration",
				*UE::GameInput::LexToString(Device));
		}
	}

#if UE_GAMEINPUT_SUPPORTS_RAW
	// Raw Input
	if (DeviceKind & AllowedGameInputKinds & GameInputKindRawDeviceReport)
	{
		const bool bIsDeviceAllowed =
			PlatformSettings->bSpecialDevicesRequireExplicitDeviceConfiguration ?
			(Settings->FindDeviceConfiguration(Info) != nullptr) :
			true;

		if (bIsDeviceAllowed)
		{
			Processors.Emplace(MakeShared<FGameInputRawDeviceProcessor>());
		}
		else
		{
			UE_LOGF(
				LogGameInput,
				Warning,
				"A raw game input device (%ls) was connected but will not be processed because it does not have an explicit device configuration",
				*UE::GameInput::LexToString(Device));
		}
	}
#endif	// UE_GAMEINPUT_SUPPORTS_RAW

	// Racing wheels
	if (DeviceKind & AllowedGameInputKinds & GameInputKindRacingWheel)
	{
		Processors.Emplace(MakeShared<FGameInputRacingWheelProcessor>());
	}

	// Arcade sticks
	if (DeviceKind & AllowedGameInputKinds & GameInputKindArcadeStick)
	{
		Processors.Emplace(MakeShared<FGameInputArcadeStickProcessor>());
	}

	// Flight sticks
	if (DeviceKind & AllowedGameInputKinds & GameInputKindFlightStick)
	{
		Processors.Emplace(MakeShared<FGameInputFlightStickProcessor>());
	}

	// If this gamepad device has any "Sensor" support, then add that as well.
#if UE_GAMEINPUT_SUPPORTS_SENSORS
	if (DeviceKind & AllowedGameInputKinds & GameInputKindSensors)
	{
		Processors.Emplace(MakeShared<FGameInputGamepadSensorDeviceProcessor>());
	}
#endif
}

const GameInputKind FGameInputDeviceContainer::ProcessInput(IGameInput* GameInput, const GameInputKind CurrentSupportedKind, const GameInputKind ProcessedKindsForPlatformUserThisFrame)
{
	GameInputKind OutProcessedInputKinds = GameInputKindUnknown;

	// If we don't have a valid IGameInputDevice, then this device has been disconnected and there
	// is no need to attempt to get any Game Input readings from it. 
	//	
	// Calling GetCurrentReading with a null IGameInputDevice will actually return _all_ game input readings, which 
	// we don't want to process. We only care about readings associated with this container's device.
	// 
	// @see https://learn.microsoft.com/en-us/gaming/gdk/_content/gc/reference/input/gameinput/interfaces/igameinput/methods/igameinput_getcurrentreading
	if (!Device)
	{
		return OutProcessedInputKinds;
	}

	const bool bDoNotProcessDuplicateCapabilitiesForSingleUser = GetDefault<UGameInputDeveloperSettings>()->bDoNotProcessDuplicateCapabilitiesForSingleUser;

#if UE_GAME_INPUT_SUPPORTS_DLI
	// We can send a Synchronization Hint to Game Input to make possible improvements to input latency with
	// Dynamic Latency Input (DLI) interface. The docs say to send this hint just before the first read of any input
	// on the device for the simulation, which will be directly below in the "GetCurrentReading" call.
	// 
	// For supported devices, DLI will then attempt to sync the reporting rate of the controller 
	// with that of the game simulation, providing better input latency.
	//
	// Note: If you call this at higher than 125hz, the DLI interface will disable itself
	{
		const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);
		
		if (Info && (Info->capabilities & GameInputDeviceCapabilitySynchronization) != 0)
		{
			Device->SendInputSynchronizationHint();
		}
	}
#endif	// #if UE_GAME_INPUT_SUPPORTS_DLI
	
	// keep reading the input snapshots for this device
	if (!LastReading.IsValid())
	{				
		GameInput->GetCurrentReading(CurrentSupportedKind, Device, &LastReading);
	}

	int32 NumReadingsProcessed = 0;

	while (LastReading.IsValid())
	{
		// The current game input reading. This can be null if there is nothing else in the input stack
		TComPtr<IGameInputReading> Reading;
		HRESULT hr = GameInput->GetNextReading(LastReading.Get(), CurrentSupportedKind, Device, &Reading);
		
		// On the last reading of the frame for this device, there will not be a "next" reading.
		const bool bIsLastReadingOfFrame = (hr == GAMEINPUT_E_READING_NOT_FOUND);

		if (FAILED(hr))
		{
			if (hr != GAMEINPUT_E_READING_NOT_FOUND)
			{
				// unexpected error - start from scratch next frame
				LastReading.Reset();
				break;
			}
		}

		// ignore this input if we're suppressing input for a while
		if (Reading && Reading->GetTimestamp() < IgnoreReadingTimestamp)
		{
			LastReading = Reading;
			return OutProcessedInputKinds;
		}

		const GameInputKind CurrentReadingKind = Reading ? Reading->GetInputKind() : LastReading->GetInputKind();

		// Pass along this reading to our Input Processors, who will actually do the work of sending messages
		// and events to the message handler
		IGameInputDeviceProcessor::FGameInputEventParams Params = {};
		Params.Reading = Reading;
		Params.PreviousReading = LastReading;
		Params.Device = Device;
		Params.MessageHandler = MessageHandler;
		Params.PlatformUserId = UserId;
		Params.InputDeviceId = AssignedDeviceId;

		for (TSharedPtr<IGameInputDeviceProcessor>& Processor : Processors)
		{
			const GameInputKind ProcessorKind = Processor->GetSupportedReadingKind();

			// The first time we process this, check for if the user has already handled a processor of this kind during this frame.
			// If they have, then we should skip processing for the rest of this frame.
			if (NumReadingsProcessed == 0 && bDoNotProcessDuplicateCapabilitiesForSingleUser && (ProcessedKindsForPlatformUserThisFrame & ProcessorKind) != 0)
			{
				continue;
			}

			// Only try and process this processor if the GameInput reading has readings for it
			if ((ProcessorKind & CurrentReadingKind) != 0)
			{
				bool bHasInputThisFrame = false;
				if (bIsLastReadingOfFrame)
				{
					// On the last reading input for the frame, the current reading pointer should be null
					ensure(!Params.Reading && Params.PreviousReading);
					bHasInputThisFrame |= Processor->PostProcessInput(Params);
				}
				else
				{
					// When processing normal input we should always have a current
					// and previous reading
					ensure(Params.Reading && Params.PreviousReading);
					bHasInputThisFrame |= Processor->ProcessInput(Params);
					++NumReadingsProcessed;
				}

				// Keep track of what input processors have sent events
				if (bHasInputThisFrame)
				{
					OutProcessedInputKinds |= ProcessorKind;
				}				
			}
		}

		// if this was the last reading of the frame, then Reading is going to be null
		// we don't need to track anything here because it happened on the previous iteration
		if (bIsLastReadingOfFrame)
		{
			break;
		}

		// Remember this reading so that we can diff against it next frame if we need to
		LastReading = Reading;

		// Keep track of the timestamp of this reading so that we can later determine the most recently used device
		LastReadingTimestamp = LastReading->GetTimestamp();
	}

	UE_CLOGF(NumReadingsProcessed > 0, LogGameInput, VeryVerbose, "Processed '%d' GameInput readings off the input stack for device: %ls", NumReadingsProcessed, *UE::GameInput::LexToString(Device));

	return OutProcessedInputKinds;
}

void FGameInputDeviceContainer::ClearInputState(IGameInput* GameInput)
{
	IGameInputDeviceProcessor::FGameInputEventParams Params = {};
	Params.Reading = nullptr;	// We have no reading when we clear input
	Params.PreviousReading = nullptr;
	Params.Device = Device;
	Params.MessageHandler = MessageHandler;
	Params.PlatformUserId = UserId;
	Params.InputDeviceId = AssignedDeviceId;

	for (TSharedPtr<IGameInputDeviceProcessor>& Processor : Processors)
	{
		Processor->ClearState(Params);
	}

	LastReading.Reset();
}

IGameInputDevice* FGameInputDeviceContainer::GetGameInputDevice() const
{
	return Device;
}

void FGameInputDeviceContainer::SetGameInputDevice(IGameInputDevice* InDevice)
{
	// If the devices are the same, we can early exit. Nothing needs to happen
	if (InDevice == Device)
	{
		return;
	}

	Device = InDevice;

	if (Device)
	{
		// Ensure that the local app device ID is the same as it was before
		// Every input device that is connected to game input has a unique App Local ID, so if we are 
		// using the same container then it should be the same.
		// 
		// In this case, we would get here if the IGameInputDevice pointer was set to null upon disconnection, 
		// and then the device was re-connected.
		if (const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device))
		{
			const bool bAppIdsAreTheSame = (FMemory::Memcmp(&Info->deviceId, &LocalDeviceId, sizeof(LocalDeviceId)) == 0);
			ensure(bAppIdsAreTheSame);
		}
	}
}

APP_LOCAL_DEVICE_ID FGameInputDeviceContainer::GetGameInputDeviceId() const
{
	return LocalDeviceId;
}

void FGameInputDeviceContainer::SetPlatformUserId(const FPlatformUserId InUserId)
{
	UserId = InUserId;
}

FPlatformUserId FGameInputDeviceContainer::GetPlatformUserId() const
{
	return UserId;
}

void FGameInputDeviceContainer::SetInputDeviceId(const FInputDeviceId InDeviceId)
{
	AssignedDeviceId = InDeviceId;
}

FInputDeviceId FGameInputDeviceContainer::GetDeviceId() const
{
	return AssignedDeviceId;
}

uint64 FGameInputDeviceContainer::GetLastReadingTimestamp() const 
{ 
	return LastReadingTimestamp; 
}

const int32 FGameInputDeviceContainer::GetNumberOfProcessors() const
{
	return Processors.Num();
}

#endif	// GAME_INPUT_SUPPORT