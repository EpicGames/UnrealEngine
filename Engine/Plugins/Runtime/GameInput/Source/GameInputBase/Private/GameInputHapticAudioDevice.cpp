// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputHapticAudioDevice.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS

#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "GameInputBaseIncludes.h"   // GAMEINPUT_HAPTIC_LOCATION_* constants
#include "GameInputLogging.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "ScopedCom.h"

namespace UE::GameInput::WasapiHaptics
{
	// Position-indexed channel-mask mapping. Matches the MS GDK Haptics sample.
	static constexpr uint32 ChannelMasks[] =
	{
		SPEAKER_FRONT_LEFT,
		SPEAKER_FRONT_RIGHT,
		SPEAKER_BACK_LEFT,
		SPEAKER_BACK_RIGHT,
	};

	// GAMEINPUT_HAPTIC_LOCATION_NONE is defined in GameInput.h as an all-zeros GUID
	// (DEFINE_GUID(..., 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0)). Rather than linking the
	// symbol (its storage only materializes when INITGUID is defined before include,
	// and no UE module does that for this header), compare against the zero pattern.
	static bool IsNoneLocation(const GUID& G)
	{
		static const GUID ZeroGuid = {};
		return FMemory::Memcmp(&G, &ZeroGuid, sizeof(GUID)) == 0;
	}
}

// ============================================================================
// FHapticRenderRunnable
// ============================================================================

uint32 FHapticRenderRunnable::Run()
{
	bIsRunning = true;

	// WASAPI vtable calls (IAudioRenderClient::GetBuffer, IAudioClient::GetCurrentPadding)
	// require COM initialized on the calling thread.
	Audio::FScopedCoInitialize ScopedCoInitialize;

	while (bIsRunning.load())
	{
		constexpr DWORD TimeoutMs = 1000;
		const DWORD Result = ::WaitForSingleObject(SampleReadyEvent, TimeoutMs);

		if (!bIsRunning.load())
		{
			break;
		}

		if (Result == WAIT_OBJECT_0)
		{
			if (Owner)
			{
				HRESULT hr = Owner->ServiceRenderCallback();
				if (hr == AUDCLNT_E_DEVICE_INVALIDATED || hr == AUDCLNT_E_RESOURCES_INVALIDATED)
				{
					Owner->RecoverFromInvalidation();
				}
			}
		}
		// WAIT_TIMEOUT / WAIT_ABANDONED — loop and re-check bIsRunning.
	}

	return 0;
}

void FHapticRenderRunnable::Stop()
{
	bIsRunning = false;
	if (SampleReadyEvent)
	{
		::SetEvent(SampleReadyEvent);
	}
}

// ============================================================================
// FGameInputHapticAudioDevice
// ============================================================================

FGameInputHapticAudioDevice::FGameInputHapticAudioDevice() = default;

FGameInputHapticAudioDevice::~FGameInputHapticAudioDevice()
{
	Teardown();
}

bool FGameInputHapticAudioDevice::Initialize(const FString& AudioEndpointId, TArrayView<const GUID> InHapticLocations)
{
	if (bIsSetup.load(std::memory_order_acquire))
	{
		Teardown();
	}

	EndpointId = AudioEndpointId;
	HapticLocations.Reset(InHapticLocations.Num());
	HapticLocations.Append(InHapticLocations.GetData(), InHapticLocations.Num());

	SampleReadyEvent = ::CreateEventEx(nullptr, nullptr, 0, EVENT_ALL_ACCESS);
	if (!SampleReadyEvent)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: CreateEventEx failed"));
		return false;
	}

	if (!InitializeInternal())
	{
		ShutdownInternal();
		if (SampleReadyEvent) { ::CloseHandle(SampleReadyEvent); SampleReadyEvent = nullptr; }
		return false;
	}

	// Cache the platform headroom gain compensation. Initialize is called from the
	// game thread (HandleHapticsReady_Impl), so we read the headroom synchronously
	// before the first OnAudioCallback can fire.
	if (GEngine)
	{
		if (FAudioDeviceHandle MainAudioDevice = GEngine->GetMainAudioDevice())
		{
			const float Headroom = FMath::Clamp(MainAudioDevice->GetPlatformAudioHeadroom(), UE_KINDA_SMALL_NUMBER, 1.f);
			OutputGain = 1.f / Headroom;
		}
	}

	const int32 QueueSize =
		UE::GameInput::EHapticAudioDefaults::HapticNumFrames *
		NumChannels *
		UE::GameInput::EHapticAudioDefaults::HapticQueueDepth + 1;
	AudioQueue = MakeUnique<TCircularQueue<float>>(QueueSize);

	// Spawn the render worker. TimeCritical matches AudioMixerWasapi's device thread priority.
	RenderRunnable = MakeUnique<FHapticRenderRunnable>(this, SampleReadyEvent);
	RenderThread.Reset(FRunnableThread::Create(
		RenderRunnable.Get(),
		TEXT("GameInputHapticRender"),
		0,
		TPri_TimeCritical));

	if (!RenderThread.IsValid())
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: Failed to create render thread"));
		RenderRunnable.Reset();
		AudioQueue.Reset();
		ShutdownInternal();
		if (SampleReadyEvent) { ::CloseHandle(SampleReadyEvent); SampleReadyEvent = nullptr; }
		return false;
	}

	bIsSetup.store(true, std::memory_order_release);

	UE_LOG(LogGameInput, Log,
		TEXT("FGameInputHapticAudioDevice: Initialized (endpoint='%s', locations=%d, channels=%d, rate=%u)"),
		*AudioEndpointId, HapticLocations.Num(), NumChannels, SampleRate);

	return true;
}

bool FGameInputHapticAudioDevice::InitializeInternal()
{
	using namespace UE::GameInput::WasapiHaptics;

	Audio::FScopedCoInitialize ScopedCoInitialize;

	TComPtr<IMMDeviceEnumerator> Enumerator;
	HRESULT hr = ::CoCreateInstance(__uuidof(MMDeviceEnumerator), nullptr, CLSCTX_ALL, IID_PPV_ARGS(&Enumerator));
	if (FAILED(hr) || !Enumerator)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: CoCreateInstance(MMDeviceEnumerator) failed (hr=0x%X)"), hr);
		return false;
	}

	hr = Enumerator->GetDevice(*EndpointId, &MMDevice);
	if (FAILED(hr) || !MMDevice)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: Enumerator->GetDevice('%s') failed (hr=0x%X)"), *EndpointId, hr);
		return false;
	}

	hr = MMDevice->Activate(__uuidof(IAudioClient2), CLSCTX_ALL, nullptr, IID_PPV_ARGS_Helper(&AudioClient));
	if (FAILED(hr) || !AudioClient)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: IMMDevice->Activate failed (hr=0x%X)"), hr);
		return false;
	}

	// AudioCategory_GameEffects matches the XAudio2 code path we are replacing and
	// signals to the audio engine that this is in-game effect audio.
	AudioClientProperties Props = {};
	Props.cbSize = sizeof(Props);
	Props.bIsOffload = false;
	Props.eCategory = AudioCategory_GameEffects;
	AudioClient->SetClientProperties(&Props);  // best-effort; some devices don't honor this

	// Build the output format from the haptic locations. Position-indexed: locations[i]
	// occupies output channel i in the interleaved stream. Skip GAMEINPUT_HAPTIC_LOCATION_NONE.
	WORD  ActiveChannels = 0;
	DWORD ChannelMask    = 0;
	for (int32 i = 0; i < HapticLocations.Num() && i < (int32)UE_ARRAY_COUNT(ChannelMasks); ++i)
	{
		if (!IsNoneLocation(HapticLocations[i]))
		{
			ChannelMask |= ChannelMasks[i];
			++ActiveChannels;
		}
	}
	if (ActiveChannels == 0)
	{
		// Degenerate case (no recognized locations) — fall back to mono FL channel.
		ActiveChannels = 1;
		ChannelMask = SPEAKER_FRONT_LEFT;
	}

	// Pull the device's sample rate from GetMixFormat so the endpoint does not have
	// to resample from an arbitrary rate. AUTOCONVERTPCM below still handles any
	// mismatch, but matching the device rate avoids extra conversion latency.
	WAVEFORMATEX* DeviceMixFormat = nullptr;
	hr = AudioClient->GetMixFormat(&DeviceMixFormat);
	if (FAILED(hr) || !DeviceMixFormat)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: GetMixFormat failed (hr=0x%X)"), hr);
		return false;
	}
	const uint32 DeviceSampleRate = DeviceMixFormat->nSamplesPerSec;
	::CoTaskMemFree(DeviceMixFormat);

	// Allocate MixFormat through CoTaskMemAlloc so ShutdownInternal's CoTaskMemFree
	// matches the allocator (consistent with formats returned by GetMixFormat).
	WAVEFORMATEXTENSIBLE* Wfx = static_cast<WAVEFORMATEXTENSIBLE*>(::CoTaskMemAlloc(sizeof(WAVEFORMATEXTENSIBLE)));
	if (!Wfx)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: CoTaskMemAlloc(WAVEFORMATEXTENSIBLE) failed"));
		return false;
	}
	FMemory::Memzero(Wfx, sizeof(WAVEFORMATEXTENSIBLE));

	Wfx->Format.wFormatTag          = WAVE_FORMAT_EXTENSIBLE;
	Wfx->Format.nChannels           = ActiveChannels;
	Wfx->Format.nSamplesPerSec      = DeviceSampleRate;
	Wfx->Format.wBitsPerSample      = 32;
	Wfx->Format.nBlockAlign         = (WORD)(ActiveChannels * (Wfx->Format.wBitsPerSample / 8));
	Wfx->Format.nAvgBytesPerSec     = Wfx->Format.nSamplesPerSec * Wfx->Format.nBlockAlign;
	Wfx->Format.cbSize              = sizeof(WAVEFORMATEXTENSIBLE) - sizeof(WAVEFORMATEX);
	Wfx->Samples.wValidBitsPerSample = Wfx->Format.wBitsPerSample;
	Wfx->dwChannelMask              = ChannelMask;
	Wfx->SubFormat                  = KSDATAFORMAT_SUBTYPE_IEEE_FLOAT;

	MixFormat = reinterpret_cast<WAVEFORMATEX*>(Wfx);

	const DWORD Flags =
		AUDCLNT_STREAMFLAGS_EVENTCALLBACK |
		AUDCLNT_STREAMFLAGS_NOPERSIST |
		AUDCLNT_STREAMFLAGS_AUTOCONVERTPCM;

	hr = AudioClient->Initialize(AUDCLNT_SHAREMODE_SHARED, Flags, 0, 0, MixFormat, nullptr);
	if (FAILED(hr))
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: IAudioClient::Initialize failed (hr=0x%X)"), hr);
		return false;
	}

	REFERENCE_TIME DefaultPeriod = 0;
	REFERENCE_TIME MinPeriod     = 0;
	if (SUCCEEDED(AudioClient->GetDevicePeriod(&DefaultPeriod, &MinPeriod)))
	{
		constexpr REFERENCE_TIME ReftimesPerSecond = 10000000LL;
		FramesPerPeriod = (uint32)((double)MixFormat->nSamplesPerSec * (double)DefaultPeriod / (double)ReftimesPerSecond + 0.5);
	}

	hr = AudioClient->GetBufferSize(&BufferFrames);
	if (FAILED(hr))
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: GetBufferSize failed (hr=0x%X)"), hr);
		return false;
	}

	hr = AudioClient->GetService(__uuidof(IAudioRenderClient), IID_PPV_ARGS_Helper(&RenderClient));
	if (FAILED(hr) || !RenderClient)
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: GetService(IAudioRenderClient) failed (hr=0x%X)"), hr);
		return false;
	}

	hr = AudioClient->SetEventHandle(SampleReadyEvent);
	if (FAILED(hr))
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: SetEventHandle failed (hr=0x%X)"), hr);
		return false;
	}

	NumChannels = ActiveChannels;
	SampleRate  = MixFormat->nSamplesPerSec;
	BlockAlign  = MixFormat->nBlockAlign;

	// Pre-roll with one buffer of silence so the first WASAPI signal has data ready,
	// matching the MS GDK Haptics sample's StartDevice behavior.
	{
		BYTE* Data = nullptr;
		if (SUCCEEDED(RenderClient->GetBuffer(BufferFrames, &Data)))
		{
			RenderClient->ReleaseBuffer(BufferFrames, AUDCLNT_BUFFERFLAGS_SILENT);
		}
	}

	hr = AudioClient->Start();
	if (FAILED(hr))
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: AudioClient->Start failed (hr=0x%X)"), hr);
		return false;
	}

	return true;
}

void FGameInputHapticAudioDevice::Teardown()
{
	// Fast-out when fully idle.
	if (!bIsSetup.load(std::memory_order_acquire) && !RenderThread.IsValid() && !SampleReadyEvent)
	{
		return;
	}

	// Block new PushAudio callers.
	bIsSetup.store(false, std::memory_order_release);

	// Stop the render runnable first (signals the event to wake the thread), then
	// join. Once Kill returns, no render-thread callback is in flight and we own
	// AudioClient / RenderClient / AudioQueue exclusively.
	if (RenderRunnable.IsValid())
	{
		RenderRunnable->Stop();
	}
	if (RenderThread.IsValid())
	{
		RenderThread->Kill(true);
		RenderThread.Reset();
	}
	RenderRunnable.Reset();

	// Drain any PushAudio caller that already passed the bIsSetup gate.
	{
		FScopeLock Lock(&StateLock);
		AudioQueue.Reset();
	}

	ShutdownInternal();

	if (SampleReadyEvent)
	{
		::CloseHandle(SampleReadyEvent);
		SampleReadyEvent = nullptr;
	}

	UE_LOG(LogGameInput, Log, TEXT("FGameInputHapticAudioDevice: Torn down"));
}

void FGameInputHapticAudioDevice::ShutdownInternal()
{
	if (AudioClient)
	{
		AudioClient->Stop();
	}
	RenderClient.Reset();
	AudioClient.Reset();
	MMDevice.Reset();

	if (MixFormat)
	{
		::CoTaskMemFree(MixFormat);
		MixFormat = nullptr;
	}

	BufferFrames    = 0;
	FramesPerPeriod = 0;
}

void FGameInputHapticAudioDevice::PushAudio(const TArrayView<const float>& InAudio, int32 InNumChannels)
{
	if (InAudio.Num() == 0 || !bIsSetup.load(std::memory_order_acquire))
	{
		return;
	}

	FScopeLock Lock(&StateLock);
	if (!bIsSetup.load(std::memory_order_acquire) || !AudioQueue)
	{
		return;
	}

	const float* Src = InAudio.GetData();

	if (InNumChannels == 1)
	{
		// Upmix mono to all haptic channels.
		for (int32 i = 0; i < InAudio.Num(); ++i)
		{
			const float Sample = Src[i] * OutputGain;
			for (int32 Ch = 0; Ch < NumChannels; ++Ch)
			{
				AudioQueue->Enqueue(Sample);
			}
		}
	}
	else if (InNumChannels == NumChannels)
	{
		for (int32 i = 0; i < InAudio.Num(); ++i)
		{
			AudioQueue->Enqueue(Src[i] * OutputGain);
		}
	}
	else
	{
		// Mismatched layout — use the first input channel and broadcast it.
		const int32 NumFrames = InAudio.Num() / InNumChannels;
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			const float Sample = Src[Frame * InNumChannels] * OutputGain;
			for (int32 Ch = 0; Ch < NumChannels; ++Ch)
			{
				AudioQueue->Enqueue(Sample);
			}
		}
	}
}

HRESULT FGameInputHapticAudioDevice::ServiceRenderCallback()
{
	if (!AudioClient || !RenderClient)
	{
		return S_OK;
	}

	UINT32 Padding = 0;
	HRESULT hr = AudioClient->GetCurrentPadding(&Padding);
	if (FAILED(hr))
	{
		return hr;
	}

	const UINT32 FramesAvailable = (BufferFrames > Padding) ? (BufferFrames - Padding) : 0;
	if (FramesAvailable == 0)
	{
		return S_OK;
	}

	BYTE* Data = nullptr;
	hr = RenderClient->GetBuffer(FramesAvailable, &Data);
	if (FAILED(hr))
	{
		return hr;
	}

	FillRenderBuffer(Data, FramesAvailable);

	return RenderClient->ReleaseBuffer(FramesAvailable, 0);
}

void FGameInputHapticAudioDevice::FillRenderBuffer(BYTE* Dst, uint32 FramesToWrite)
{
	float* Out = reinterpret_cast<float*>(Dst);
	const uint32 TotalFloats = FramesToWrite * (uint32)NumChannels;

	// AudioQueue destruction is sequenced after the render thread is joined in
	// Teardown, so Dequeue here cannot race with Reset(). TCircularQueue is SPSC —
	// PushAudio is the sole producer, this is the sole consumer.
	if (!AudioQueue)
	{
		FMemory::Memzero(Out, TotalFloats * sizeof(float));
		return;
	}

	for (uint32 i = 0; i < TotalFloats; ++i)
	{
		float Value = 0.f;
		AudioQueue->Dequeue(Value);  // leaves Value at 0 on underrun
		Out[i] = Value;
	}
}

void FGameInputHapticAudioDevice::RecoverFromInvalidation()
{
	if (!bIsSetup.load(std::memory_order_acquire))
	{
		return;
	}

	UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: Recovering from device invalidation"));
	ShutdownInternal();

	if (!InitializeInternal())
	{
		UE_LOG(LogGameInput, Warning, TEXT("FGameInputHapticAudioDevice: Recovery InitializeInternal failed"));
		// Next ServiceRenderCallback guards on null AudioClient and returns S_OK,
		// so the render loop idles on its 1s wait timeout until Teardown fires.
	}
}

#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
