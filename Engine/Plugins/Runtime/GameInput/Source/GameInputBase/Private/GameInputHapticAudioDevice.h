// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS

#include "Containers/CircularQueue.h"
#include "HAL/Runnable.h"
#include "IAudioEndpoint.h"
#include "GameInputHapticEndpointSettings.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>
#include <audioclient.h>
#include <ksmedia.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

class FRunnableThread;
class FGameInputHapticAudioDevice;

namespace UE::GameInput
{
	enum EHapticAudioDefaults : int32
	{
		HapticSampleRate  = 48000,
		HapticNumFrames   = 256,
		HapticQueueDepth  = 4,
	};
}

// ----------------------------------------------------------------------------
// FHapticRenderRunnable
//
// Worker-thread loop that waits on WASAPI's sample-ready event and forwards
// each wakeup to the owning device's render-side service function.
// ----------------------------------------------------------------------------
class FHapticRenderRunnable final : public FRunnable
{
public:
	explicit FHapticRenderRunnable(FGameInputHapticAudioDevice* InOwner, HANDLE InSampleReadyEvent)
		: Owner(InOwner)
		, SampleReadyEvent(InSampleReadyEvent)
	{}

	virtual uint32 Run() override;
	virtual void   Stop() override;

private:
	FGameInputHapticAudioDevice* Owner = nullptr;
	HANDLE                       SampleReadyEvent = nullptr;
	std::atomic_bool             bIsRunning { false };
};

// ----------------------------------------------------------------------------
// FGameInputHapticAudioDevice
//
// Owns a WASAPI IAudioClient2 targeting one GameInput haptic controller's
// audio endpoint. Created by FGameInputHapticEndpointFactory and initialized
// once GameInputHapticInfo is available from HandleHapticsReady_Impl.
// ----------------------------------------------------------------------------
class FGameInputHapticAudioDevice
{
public:

	FGameInputHapticAudioDevice();
	~FGameInputHapticAudioDevice();

	/**
	 * Set up WASAPI targeting the given endpoint using the supplied haptic locations
	 * to build a WAVEFORMATEXTENSIBLE channel mask. Safe to call multiple times;
	 * re-initializes on repeated calls.
	 *
	 * @param AudioEndpointId     The WASAPI device ID from GameInputHapticInfo::audioEndpointId.
	 * @param InHapticLocations   GameInputHapticLocation GUIDs from GameInputHapticInfo::locations,
	 *                            sized to locationCount. The count also becomes the render-side
	 *                            channel count.
	 * @return true if WASAPI was successfully initialized.
	 */
	bool Initialize(const FString& AudioEndpointId, TArrayView<const GUID> InHapticLocations);

	/** Shut down WASAPI and release all resources. Safe to call when not initialized. */
	void Teardown();

	bool IsInitialized() const { return bIsSetup.load(std::memory_order_acquire); }
	int32 GetNumChannels() const { return NumChannels; }
	uint32 GetSampleRate() const { return SampleRate; }

	/**
	 * Enqueue audio samples. Called from the UE audio thread via FGameInputHapticEndpoint::OnAudioCallback.
	 * Handles mono→N-channel upmix and applies platform headroom gain compensation.
	 */
	void PushAudio(const TArrayView<const float>& InAudio, int32 InNumChannels);

private:

	friend class FHapticRenderRunnable;

	// Runs the CoCreateInstance → Activate → Initialize → Start sequence using the
	// cached EndpointId + HapticLocations. Called both by Initialize() on the game
	// thread (first boot) and by RecoverFromInvalidation() on the render thread
	// (after AUDCLNT_E_*_INVALIDATED).
	bool InitializeInternal();

	// Release COM pointers and MixFormat. Does not touch events, threads, or the queue.
	void ShutdownInternal();

	// Render-thread callback invoked when SampleReadyEvent fires.
	HRESULT ServiceRenderCallback();

	// Drain AudioQueue into the WASAPI render buffer; zero-fill on underrun.
	void FillRenderBuffer(BYTE* Dst, uint32 FramesToWrite);

	// Tear down and re-run InitializeInternal(). Called on the render thread when
	// ServiceRenderCallback returns AUDCLNT_E_*_INVALIDATED.
	void RecoverFromInvalidation();

	// --- WASAPI COM state ----------------------------------------------------
	TComPtr<IMMDevice>          MMDevice;
	TComPtr<IAudioClient2>      AudioClient;
	TComPtr<IAudioRenderClient> RenderClient;
	WAVEFORMATEX*               MixFormat      = nullptr;   // CoTaskMemFree on teardown
	uint32                      BufferFrames   = 0;
	uint32                      FramesPerPeriod = 0;

	// --- Render worker thread ------------------------------------------------
	HANDLE                      SampleReadyEvent = nullptr;   // auto-reset event; WASAPI signals it
	TUniquePtr<FRunnableThread> RenderThread;
	TUniquePtr<FHapticRenderRunnable> RenderRunnable;

	// --- Cached endpoint config for restart-on-invalidation ------------------
	FString                     EndpointId;
	TArray<GUID>                HapticLocations;

	// --- Shared state --------------------------------------------------------
	// Protects AudioQueue access between the UE audio thread (PushAudio) and the
	// game thread (Teardown). The render thread reads AudioQueue only after
	// InitializeInternal succeeds; Teardown joins the render thread before the
	// queue can be destroyed, so the render side does not need StateLock.
	FCriticalSection            StateLock;

	TUniquePtr<TCircularQueue<float>> AudioQueue;

	int32                       NumChannels   = 2;
	uint32                      SampleRate    = 48000;
	uint32                      BlockAlign    = sizeof(float) * 2;
	float                       OutputGain    = 1.f;
	std::atomic_bool            bIsSetup      { false };
};

// ----------------------------------------------------------------------------
// FGameInputHapticEndpoint
//
// IAudioEndpoint implementation that forwards UE audio mixer callbacks to
// FGameInputHapticAudioDevice. One instance is created per UEndpointSubmix
// that uses the "Vibration Output" endpoint type.
// ----------------------------------------------------------------------------
class FGameInputHapticEndpoint : public IAudioEndpoint
{
public:

	explicit FGameInputHapticEndpoint(TSharedRef<FGameInputHapticAudioDevice> InDevice)
		: Device(MoveTemp(InDevice))
	{}

	virtual float GetSampleRate() const override
	{
		// Return the engine sample rate so the UE mixer does NOT create a
		// resampler — matching WinDualShock's approach. WASAPI handles the
		// final rate conversion to whatever the controller's endpoint
		// negotiates (e.g. 8000 Hz on Xbox) via AUDCLNT_STREAMFLAGS_AUTOCONVERTPCM.
		return static_cast<float>(UE::GameInput::EHapticAudioDefaults::HapticSampleRate);
	}

	virtual int32 GetNumChannels() const override
	{
		// Must return a stable constant from the moment the mixer reads it during submix setup.
		// Returning Device->GetNumChannels() here would cause a check() failure in OnAudioCallback
		// if the device's channel count changes after Initialize() is called (e.g. Xbox negotiates
		// 4 haptic channels after the controller connects). The mixer configures the submix's
		// channel layout once and never re-reads this — match WinDualShock's VibrationChannels = 2.
		return 2;
	}

	virtual bool IsImplemented() override
	{
		return true;
	}

	virtual bool EndpointRequiresCallback() const override
	{
		return true;
	}

	virtual int32 GetDesiredNumFrames() const override
	{
		return UE::GameInput::EHapticAudioDefaults::HapticNumFrames;
	}

	virtual bool OnAudioCallback(
		const TArrayView<const float>& InAudio,
		const int32& NumChannels,
		const IAudioEndpointSettingsProxy* InSettings) override
	{
		check(NumChannels == GetNumChannels());
		Device->PushAudio(InAudio, NumChannels);
		return true;
	}

private:

	TSharedRef<FGameInputHapticAudioDevice> Device;
};

#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
