// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputHapticEndpointFactory.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS

#include "IAudioExtensionPlugin.h"
#include "AudioDevice.h"
#include "GameInputHapticEndpointSettings.h"
#include "GameInputLogging.h"

// ============================================================================
// FGameInputHapticEndpointFactory
// ============================================================================

namespace UE
{
	static const FName VibrationOutputName = TEXT("Vibration Output");
}

FGameInputHapticEndpointFactory::FGameInputHapticEndpointFactory()
{
	// check for an existing endpoint of the same name ( "Vibration Output" )
	// The WinDualShock plugin is also a "vibration output" name which may be active, and we don't
	// want to make this endpoint if that one already exists.
	const IAudioEndpointFactory* ExistingEndpoint = IAudioEndpointFactory::Get(UE::VibrationOutputName);
	if (ExistingEndpoint && ExistingEndpoint->bIsImplemented)
	{
		UE_LOGF(LogGameInput, Warning, "FGameInputHapticEndpointFactory will not be registered! There is an existing endpoint with the name '%ls'", *UE::VibrationOutputName.ToString());
	}
	else
	{
		IAudioEndpointFactory::RegisterEndpointType(this);
		bIsImplemented = true;	
	}
}

FGameInputHapticEndpointFactory::~FGameInputHapticEndpointFactory()
{
	IAudioEndpointFactory::UnregisterEndpointType(this);
}

FName FGameInputHapticEndpointFactory::GetEndpointTypeName()
{
	// Use the same platform-agnostic name as WinDualShock so existing content
	// ("Vibration Output" endpoint submixes) routes here on GameInput platforms
	// without requiring asset changes.
	return UE::VibrationOutputName;
}

TUniquePtr<IAudioEndpoint> FGameInputHapticEndpointFactory::CreateNewEndpointInstance(
	const FAudioPluginInitializationParams& InitInfo,
	const IAudioEndpointSettingsProxy& InitialSettings)
{
	// The physical GameInput device may not have connected yet, so we use
	// INPUTDEVICEID_NONE as a placeholder. InitializeDevice() (called from
	// HandleHapticsReady_Impl on the game thread) will re-key this entry once
	// the device connects. Keying by AudioDeviceId isolates editor and PIE
	// audio devices from each other.
	check(InitInfo.AudioDevicePtr);
	const Audio::FDeviceId AudioDeviceId = InitInfo.AudioDevicePtr->DeviceID;
	TSharedRef<FGameInputHapticAudioDevice> Device = GetOrCreatePlaceholder(AudioDeviceId);
	return MakeUnique<FGameInputHapticEndpoint>(Device);
}

UClass* FGameInputHapticEndpointFactory::GetCustomSettingsClass() const
{
	return UGameInputHapticEndpointSettings::StaticClass();
}

const UAudioEndpointSettingsBase* FGameInputHapticEndpointFactory::GetDefaultSettings() const
{
	return GetDefault<UGameInputHapticEndpointSettings>();
}

/** Returns true if the given APP_LOCAL_DEVICE_ID is all zeroes (the placeholder state). */
static bool IsZeroDeviceId(const APP_LOCAL_DEVICE_ID& InId)
{
	static const APP_LOCAL_DEVICE_ID Zero = {};
	return FMemory::Memcmp(&InId, &Zero, sizeof(APP_LOCAL_DEVICE_ID)) == 0;
}

TSharedRef<FGameInputHapticAudioDevice> FGameInputHapticEndpointFactory::GetOrCreatePlaceholder(Audio::FDeviceId AudioDeviceId)
{
	FString EndpointIdToInit;
	TArray<GUID> LocationsToInit;

	TSharedRef<FGameInputHapticAudioDevice> Device = [&]() -> TSharedRef<FGameInputHapticAudioDevice>
	{
		FScopeLock Lock(&DeviceMapCS);

		const FHapticDeviceKey PlaceholderKey = { AudioDeviceId, {} };
		if (TSharedRef<FGameInputHapticAudioDevice>* Existing = DeviceMap.Find(PlaceholderKey))
		{
			return *Existing;
		}

		TSharedRef<FGameInputHapticAudioDevice> NewDevice = MakeShared<FGameInputHapticAudioDevice>();

		// If a controller is already connected (ActiveDeviceId is non-zero), key
		// directly with the real device ID instead of creating a placeholder.
		// This handles late-arriving audio devices (e.g. PIE starting after the
		// controller is already connected).
		if (!IsZeroDeviceId(ActiveDeviceId))
		{
			const FHapticDeviceKey ActiveKey = { AudioDeviceId, ActiveDeviceId };
			DeviceMap.Add(ActiveKey, NewDevice);
			EndpointIdToInit = ActiveAudioEndpointId;
			LocationsToInit = ActiveHapticLocations;
		}
		else
		{
			DeviceMap.Add(PlaceholderKey, NewDevice);
		}

		return NewDevice;
	}();

	// Initialize outside the lock if we have active endpoint info.
	if (!EndpointIdToInit.IsEmpty())
	{
		Device->Initialize(EndpointIdToInit, LocationsToInit);
	}

	return Device;
}

bool FGameInputHapticEndpointFactory::IsDeviceInitialized(const APP_LOCAL_DEVICE_ID& InDeviceId) const
{
	FScopeLock Lock(&DeviceMapCS);

	for (const TPair<FHapticDeviceKey, TSharedRef<FGameInputHapticAudioDevice>>& Pair : DeviceMap)
	{
		if (FMemory::Memcmp(&Pair.Key.AppLocalDeviceId, &InDeviceId, sizeof(APP_LOCAL_DEVICE_ID)) == 0
			&& Pair.Value->IsInitialized())
		{
			return true;
		}
	}

	return false;
}

void FGameInputHapticEndpointFactory::InitializeDevice(const APP_LOCAL_DEVICE_ID& InDeviceId, const FString& AudioEndpointId, TArrayView<const GUID> HapticLocations)
{
	// Re-key all placeholder entries (zero APP_LOCAL_DEVICE_ID) to the real device ID.
	// Each UE audio device (editor / PIE) gets its own WASAPI client instance targeting
	// the same endpoint. Live endpoints already hold TSharedRefs to their
	// FGameInputHapticAudioDevice, so re-keying is safe after the fact.
	TArray<TSharedRef<FGameInputHapticAudioDevice>> DevicesToInitialize;
	{
		FScopeLock Lock(&DeviceMapCS);

		// Cache the endpoint info so late-arriving audio devices (e.g. PIE starting
		// after the controller is already connected) can be initialized immediately
		// in GetOrCreatePlaceholder.
		ActiveDeviceId = InDeviceId;
		ActiveAudioEndpointId = AudioEndpointId;
		ActiveHapticLocations.Reset(HapticLocations.Num());
		ActiveHapticLocations.Append(HapticLocations.GetData(), HapticLocations.Num());

		TArray<FHapticDeviceKey> PlaceholderKeys;
		for (TPair<FHapticDeviceKey, TSharedRef<FGameInputHapticAudioDevice>>& Pair : DeviceMap)
		{
			if (IsZeroDeviceId(Pair.Key.AppLocalDeviceId))
			{
				PlaceholderKeys.Add(Pair.Key);
			}
		}

		for (const FHapticDeviceKey& OldKey : PlaceholderKeys)
		{
			TSharedRef<FGameInputHapticAudioDevice> Device = DeviceMap.FindAndRemoveChecked(OldKey);
			const FHapticDeviceKey NewKey = { OldKey.AudioDeviceId, InDeviceId };
			DeviceMap.Add(NewKey, Device);
			DevicesToInitialize.Add(Device);
		}
	}

	// Initialize outside the lock — WASAPI activation does not need DeviceMapCS.
	for (TSharedRef<FGameInputHapticAudioDevice>& Device : DevicesToInitialize)
	{
		Device->Initialize(AudioEndpointId, HapticLocations);
	}

	UE_LOG(LogGameInput, Log,
		TEXT("FGameInputHapticEndpointFactory::InitializeDevice — endpoint='%s', locations=%d, devicesInitialized=%d"),
		*AudioEndpointId, HapticLocations.Num(), DevicesToInitialize.Num());
}

void FGameInputHapticEndpointFactory::RemoveDevice(const APP_LOCAL_DEVICE_ID& InDeviceId)
{
	// Re-key matching entries back to placeholder state (zero APP_LOCAL_DEVICE_ID)
	// instead of removing them. The live FGameInputHapticEndpoint objects (owned by
	// the submix) still hold TSharedRefs to these devices. On reconnect,
	// InitializeDevice will find the placeholders and re-initialize them.
	TArray<TSharedRef<FGameInputHapticAudioDevice>> DevicesToTeardown;

	{
		FScopeLock Lock(&DeviceMapCS);

		// Clear cached endpoint info so new placeholders aren't auto-initialized
		// with a stale endpoint that no longer exists.
		ActiveDeviceId = {};
		ActiveAudioEndpointId.Empty();
		ActiveHapticLocations.Reset();

		TArray<FHapticDeviceKey> KeysToRekey;
		for (TPair<FHapticDeviceKey, TSharedRef<FGameInputHapticAudioDevice>>& Pair : DeviceMap)
		{
			if (FMemory::Memcmp(&Pair.Key.AppLocalDeviceId, &InDeviceId, sizeof(APP_LOCAL_DEVICE_ID)) == 0)
			{
				KeysToRekey.Add(Pair.Key);
			}
		}

		for (const FHapticDeviceKey& OldKey : KeysToRekey)
		{
			TSharedRef<FGameInputHapticAudioDevice> Device = DeviceMap.FindAndRemoveChecked(OldKey);
			const FHapticDeviceKey PlaceholderKey = { OldKey.AudioDeviceId, {} };
			DeviceMap.Add(PlaceholderKey, Device);
			DevicesToTeardown.Add(Device);
		}
	}

	// Teardown outside the lock so XAudio2 destruction doesn't hold the CS.
	for (TSharedRef<FGameInputHapticAudioDevice>& Device : DevicesToTeardown)
	{
		Device->Teardown();
	}
}

#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
