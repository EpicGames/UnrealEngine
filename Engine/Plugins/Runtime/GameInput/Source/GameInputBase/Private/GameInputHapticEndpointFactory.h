// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS

#include "IAudioEndpoint.h"
#include "AudioDefines.h"
#include "GameInputHapticAudioDevice.h"

// ============================================================================
// FHapticDeviceKey
//
// Composite key for DeviceMap: one entry per (UE audio device × GameInput input
// device) pair. This mirrors WinDualShock's FDeviceKey and correctly isolates
// endpoints across multiple simultaneous FAudioDevice instances (e.g. editor +
// PIE), each of which calls CreateNewEndpointInstance independently.
//
// Uses APP_LOCAL_DEVICE_ID (the Windows GDK hardware identifier) rather than
// FInputDeviceId so we don't depend on the engine's input device mapper having
// assigned an ID before haptic audio initialization.
// ============================================================================
struct FHapticDeviceKey
{
	/** UE audio device that owns the endpoint submix (from FAudioPluginInitializationParams). */
	Audio::FDeviceId AudioDeviceId = 0;

	/** GameInput physical device. Zero-initialized while awaiting HandleHapticsReady. */
	APP_LOCAL_DEVICE_ID AppLocalDeviceId = {};

	bool operator==(const FHapticDeviceKey& Other) const
	{
		return AudioDeviceId == Other.AudioDeviceId
			&& FMemory::Memcmp(&AppLocalDeviceId, &Other.AppLocalDeviceId, sizeof(APP_LOCAL_DEVICE_ID)) == 0;
	}
};

inline uint32 GetTypeHash(const FHapticDeviceKey& Key)
{
	return HashCombine(Key.AudioDeviceId, FCrc::MemCrc32(&Key.AppLocalDeviceId, sizeof(APP_LOCAL_DEVICE_ID)));
}

/**
 * IAudioEndpointFactory implementation for GameInput haptic audio.
 *
 * Registers as the "Vibration Output" endpoint type so that existing
 * UEndpointSubmix content (created for WinDualShock) routes here on GameInput
 * platforms without requiring asset changes.
 *
 * DeviceMap is keyed by FHapticDeviceKey { AudioDeviceId, InputDeviceId }.
 * CreateNewEndpointInstance (audio thread) stores a placeholder entry with
 * InputDeviceId == INPUTDEVICEID_NONE. InitializeDevice (game thread, fired from
 * HandleHapticsReady_Impl) re-keys all placeholder entries and calls Initialize()
 * on each FGameInputHapticAudioDevice so every live UE audio device's endpoint
 * starts receiving haptic audio.
 *
 * Owned by FGameInputBaseModule. Registered at construction, unregistered at destruction.
 */
class FGameInputHapticEndpointFactory : public IAudioEndpointFactory
{
public:

	FGameInputHapticEndpointFactory();
	virtual ~FGameInputHapticEndpointFactory() override;

	//~ Begin IAudioEndpointFactory interface
	virtual FName GetEndpointTypeName() override;
	virtual TUniquePtr<IAudioEndpoint> CreateNewEndpointInstance(
		const FAudioPluginInitializationParams& InitInfo,
		const IAudioEndpointSettingsProxy& InitialSettings) override;
	virtual UClass* GetCustomSettingsClass() const override;
	virtual const UAudioEndpointSettingsBase* GetDefaultSettings() const override;
	//~ End IAudioEndpointFactory interface

	/**
	 * Finds or creates a placeholder FGameInputHapticAudioDevice for the given UE audio device.
	 * Called from CreateNewEndpointInstance on the audio thread.
	 */
	TSharedRef<FGameInputHapticAudioDevice> GetOrCreatePlaceholder(Audio::FDeviceId AudioDeviceId);

	/** Returns true if the given device already has initialized haptic audio endpoints. */
	bool IsDeviceInitialized(const APP_LOCAL_DEVICE_ID& InDeviceId) const;

	/**
	 * Called from the game thread (HandleHapticsReady_Impl) when a haptic controller connects.
	 * Re-keys all placeholder entries to the real APP_LOCAL_DEVICE_ID and calls Initialize()
	 * on each FGameInputHapticAudioDevice.
	 *
	 * @param HapticLocations  The GameInputHapticLocation GUID array from GameInputHapticInfo,
	 *                         sized to locationCount. Used to build the WASAPI channel mask.
	 */
	void InitializeDevice(const APP_LOCAL_DEVICE_ID& InDeviceId, const FString& AudioEndpointId, TArrayView<const GUID> HapticLocations);

	/**
	 * Called from the game thread on disconnect. Tears down and removes all entries
	 * for the given device (across all UE audio devices).
	 */
	void RemoveDevice(const APP_LOCAL_DEVICE_ID& InDeviceId);

private:

	/** Guards DeviceMap and active device state — written from game thread and audio thread. */
	mutable FCriticalSection DeviceMapCS;

	TMap<FHapticDeviceKey, TSharedRef<FGameInputHapticAudioDevice>> DeviceMap;

	/** Cached endpoint info from the last successful InitializeDevice call.
	 *  Used to immediately initialize late-arriving audio devices (e.g. PIE
	 *  starts after the controller is already connected).
	 *
	 *  NOTE: Only one active device is tracked at a time. If multiple haptic
	 *  controllers connect simultaneously, the last InitializeDevice call wins
	 *  and RemoveDevice clears the cache unconditionally. Expand these to a
	 *  TMap<APP_LOCAL_DEVICE_ID, ...> if multi-device haptic audio is needed. */
	APP_LOCAL_DEVICE_ID ActiveDeviceId = {};
	FString ActiveAudioEndpointId;
	TArray<GUID> ActiveHapticLocations;
};

#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
