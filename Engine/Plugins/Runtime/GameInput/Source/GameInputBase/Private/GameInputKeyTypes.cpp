// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputKeyTypes.h"

//        
// Racing Wheel
//        

// Analog types
const FKey FGameInputKeys::RacingWheel_Brake("GameInput_RacingWheel_Brake");
const FKey FGameInputKeys::RacingWheel_Clutch("GameInput_RacingWheel_Clutch");
const FKey FGameInputKeys::RacingWheel_Handbrake("GameInput_RacingWheel_Handbrake");
const FKey FGameInputKeys::RacingWheel_Throttle("GameInput_RacingWheel_Throttle");
const FKey FGameInputKeys::RacingWheel_Wheel("GameInput_RacingWheel_Wheel");
const FKey FGameInputKeys::RacingWheel_PatternShifterGear("GameInput_RacingWheel_PatternShifterGear");

// Button types
const FKey FGameInputKeys::RacingWheel_None("GameInput_RacingWheel_None");
const FKey FGameInputKeys::RacingWheel_Menu("GameInput_RacingWheel_Menu");
const FKey FGameInputKeys::RacingWheel_View("GameInput_RacingWheel_View");
const FKey FGameInputKeys::RacingWheel_PreviousGear("GameInput_RacingWheel_PreviousGear");
const FKey FGameInputKeys::RacingWheel_NextGear("GameInput_RacingWheel_NextGear");

//
// Flight Stick
//

// Analog Types
const FKey FGameInputKeys::FlightStick_Roll("FlightStick_Roll");
const FKey FGameInputKeys::FlightStick_Pitch("FlightStick_Pitch");
const FKey FGameInputKeys::FlightStick_Yaw("FlightStick_Yaw");
const FKey FGameInputKeys::FlightStick_Throttle("FlightStick_Throttle");

// Button types
const FKey FGameInputKeys::FlightStick_None("GameInput_FlightStick_None");
const FKey FGameInputKeys::FlightStick_Menu("GameInput_FlightStick_Menu");
const FKey FGameInputKeys::FlightStick_View("GameInput_FlightStick_View");
const FKey FGameInputKeys::FlightStick_FirePrimary("GameInput_FlightStick_FirePrimary");
const FKey FGameInputKeys::FlightStick_FireSecondary("GameInput_FlightStick_FireSecondary");

//
// Arcade Stick
//

// Button Types
const FKey FGameInputKeys::ArcadeStick_Action1("GameInput_ArcadeStick_Action1");
const FKey FGameInputKeys::ArcadeStick_Action2("GameInput_ArcadeStick_Action2");
const FKey FGameInputKeys::ArcadeStick_Action3("GameInput_ArcadeStick_Action3");
const FKey FGameInputKeys::ArcadeStick_Action4("GameInput_ArcadeStick_Action4");
const FKey FGameInputKeys::ArcadeStick_Action5("GameInput_ArcadeStick_Action5");
const FKey FGameInputKeys::ArcadeStick_Action6("GameInput_ArcadeStick_Action6");
const FKey FGameInputKeys::ArcadeStick_Special1("GameInput_ArcadeStick_Special1");
const FKey FGameInputKeys::ArcadeStick_Special2("GameInput_ArcadeStick_Special2");

//
// Sensor Types
//

// Analog Types
const FKey FGameInputKeys::Sensor_HeadingInDegreesFromMagneticNorth("GameInput_Sensor_HeadingInDegreesFromMagneticNorth");