// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputLogging.h"

DEFINE_LOG_CATEGORY(LogGameInput);