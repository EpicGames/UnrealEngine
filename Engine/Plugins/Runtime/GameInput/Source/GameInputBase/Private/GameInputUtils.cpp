// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputUtils.h"

#include "GameInputBaseModule.h"
#include "GameInputDeveloperSettings.h"
#include "Misc/Paths.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
#include "GameInputHapticEndpointFactory.h"
#endif

#if GAME_INPUT_SUPPORT

#if PLATFORM_WINDOWS
#include "Windows/WindowsRedistributableValidation.h"
#endif

namespace UE::GameInput
{
	// Note: "VersionInfo" only exist on Windows.
#if PLATFORM_WINDOWS
	/**
	* Returns the version info of the valid GameInputRedist.dll if one is available. 
	*/
	VersionInfo GetInstalledGameInputVersionInfo()
	{		 		
		const FString SystemRoot = FWindowsPlatformMisc::GetEnvironmentVariable(TEXT("SystemRoot"));
    	const FString System32Path = FPaths::Combine(SystemRoot, TEXT("system32"));
    
		auto GetDLLVersionInfo = [](const TCHAR* ExecDirectory, const TCHAR* Name) -> VersionInfo
		{
			const FString FilePath = FPaths::Combine(ExecDirectory, Name);
			if (!FilePath.IsEmpty())
			{
				const uint64 FileVersion = FWindowsPlatformMisc::GetFileVersion(FilePath);
				return VersionInfo{ FileVersion };
			}
			return false;
		};
		
		// We always prefer the version in System32 it is valid.
		const VersionInfo System32Version = GetDLLVersionInfo(*System32Path, TEXT("GameInputRedist.dll"));
		if (IsVersionValid(System32Version, MinGameInputRedistVersion))
		{
			return System32Version;
		}
		
		// Otherwise we fall back to the local dll
		const VersionInfo LocalVersion = GetDLLVersionInfo(nullptr, TEXT("GameInputRedist.dll"));
		if (IsVersionValid(LocalVersion, MinGameInputRedistVersion))
		{
			return LocalVersion;
		}

		return VersionInfo();
	}
#endif

	static const FString InvalidVersionString = TEXT("Unknown");
	
	FString GetInstalledGameInputVersionInfoString()
	{
#if PLATFORM_WINDOWS
		const VersionInfo InstalledVersion = UE::GameInput::GetInstalledGameInputVersionInfo();
		
		return FString::Printf(
			TEXT("%lu.%lu.%lu.%lu"), 
			InstalledVersion.Major, InstalledVersion.Minor, InstalledVersion.Bld, InstalledVersion.Rbld);
#else
		return InvalidVersionString;
#endif
	}
	
	FString GetMinVersionOfGameInputRedistString()
	{
#if PLATFORM_WINDOWS
		return FString::Printf(
    		TEXT("%lu.%lu.%lu.%lu"), 
    		MinGameInputRedistVersion.Major, MinGameInputRedistVersion.Minor, MinGameInputRedistVersion.Bld, MinGameInputRedistVersion.Rbld);
#else
		return InvalidVersionString;
#endif
	}

	bool HasValidVersionOfGameInput()
	{
#if PLATFORM_WINDOWS		
		auto IsDllVersionValid = [](const TCHAR* ExecDirectory, const TCHAR* Name, const VersionInfo& MinRedistVersionToCheck) -> bool
			{
				const FString FilePath = FPaths::Combine(ExecDirectory, Name);
				if (!FilePath.IsEmpty())
				{
					const uint64 FileVersion = FWindowsPlatformMisc::GetFileVersion(FilePath);
					if (FileVersion != 0)
					{
						const VersionInfo CurrentVersionInfo { FileVersion };
						return IsVersionValid(CurrentVersionInfo, MinRedistVersionToCheck);
					}
				}
				return false;
			};

		// Reproducing verifications done in BootstrapPackagedGame.cpp
		bool bHasValidRedistInputDLL = false;
		
		const FString SystemRoot = FWindowsPlatformMisc::GetEnvironmentVariable(TEXT("SystemRoot"));
		const FString System32Path = FPaths::Combine(SystemRoot, TEXT("system32"));

		// Check GameInputRedist.dll
		if (IsDllVersionValid(*System32Path, TEXT("GameInputRedist.dll"), MinGameInputRedistVersion) ||
			IsDllVersionValid(nullptr, TEXT("GameInputRedist.dll"), MinGameInputRedistVersion))
		{
			bHasValidRedistInputDLL = true;
		}
		
		return bHasValidRedistInputDLL;
#else
		return true;
#endif
	}

	FString LexToString(IGameInputDevice* Device)
	{
		FString Result = TEXT("<invalid device>");

		if (Device)
		{
			const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);
			const bool bIsVirtualDevice = Info->deviceFamily == GameInputFamilyVirtual;

#if GAMEINPUT_API_VERSION <= 0
			const FString DeviceDisplayName =
				Info->displayName ?
				FString::Printf(TEXT("%.*hs"), Info->displayName->sizeInBytes, Info->displayName->data) :
				TEXT("Unknown");
#else
			const FString DeviceDisplayName =
				Info->displayName ?
				FString::Printf(TEXT("%s"), ANSI_TO_TCHAR(Info->displayName)) :
				TEXT("Unknown");
#endif

			// If we have a custom device setup for this, print that name too
			const UGameInputDeveloperSettings* Settings = GetDefault<UGameInputDeveloperSettings>();

			const FGameInputDeviceConfiguration* CustomDeviceConfig = Settings->FindDeviceConfiguration(Info);
			const FString CustomHardwareId =
				CustomDeviceConfig ?
				FString::Printf(TEXT("(%s)"), *CustomDeviceConfig->OverriddenHardwareDeviceId) :
				TEXT("");
			
			Result = FString::Printf(TEXT("%p   DeviceName: %s %s %s ProdId: %04x (%u) VendId: %04x (%u)"),
				Device, 
				*DeviceDisplayName,
				bIsVirtualDevice ? TEXT("(virtual)") : TEXT(""),
				*CustomHardwareId,
				Info->productId, (uint32)Info->productId,
				Info->vendorId, (uint32)Info->vendorId);
		}

		return Result;
	}
	
	FString LexToString(GameInputDeviceStatus DeviceStatus)
	{
		if (DeviceStatus == GameInputDeviceNoStatus)
		{
			return TEXT("NoStatus");
		}
		else if (DeviceStatus == GameInputDeviceAnyStatus)
		{
			return TEXT("Any");
		}

		FString Result = TEXT("");
#define DEVICE_STATUS(StatusFlag,DisplayName) if( DeviceStatus & StatusFlag ) Result += (FString(DisplayName) + TEXT("|"));
		DEVICE_STATUS(GameInputDeviceConnected, TEXT("Connected"));
		
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
		DEVICE_STATUS(GameInputDeviceHapticInfoReady, TEXT("HapticInfoReady"));
#endif
		
#if UE_GAMEINPUT_SUPPORTS_DEVICE_STATUS
		DEVICE_STATUS(GameInputDeviceInputEnabled, TEXT("InputEnabled"));
		DEVICE_STATUS(GameInputDeviceOutputEnabled, TEXT("OutputEnabled"));
		DEVICE_STATUS(GameInputDeviceRawIoEnabled, TEXT("RawIoEnabled"));
		DEVICE_STATUS(GameInputDeviceAudioCapture, TEXT("AudioCapture"));
		DEVICE_STATUS(GameInputDeviceAudioRender, TEXT("AudioRender"));
		DEVICE_STATUS(GameInputDeviceSynchronized, TEXT("Synchronized"));
		DEVICE_STATUS(GameInputDeviceWireless, TEXT("Wireless"));
		DEVICE_STATUS(GameInputDeviceUserIdle, TEXT("UserIdle"));
#endif
#undef DEVICE_STATUS

		Result.RemoveFromEnd(TEXT("|"));
		return Result;
	}
	
	FString LexToString(GameInputKind InputKind)
	{
		if (InputKind == GameInputKindUnknown)
		{
			return TEXT("Unknown");
		}

		FString Result = TEXT("");
#define INPUT_KIND_STRING(StatusFlag, DisplayName) if( InputKind & StatusFlag ) Result += (FString(DisplayName) + TEXT("|"));
#if UE_GAMEINPUT_SUPPORTS_RAW
		INPUT_KIND_STRING(GameInputKindRawDeviceReport, TEXT("RawDeviceReport"));
#endif
		INPUT_KIND_STRING(GameInputKindControllerAxis, TEXT("ControllerAxis"));
		INPUT_KIND_STRING(GameInputKindControllerButton, TEXT("ControllerButton"));
		INPUT_KIND_STRING(GameInputKindControllerSwitch, TEXT("ControllerSwitch"));
		INPUT_KIND_STRING(GameInputKindController, TEXT("Controller"));
		INPUT_KIND_STRING(GameInputKindKeyboard, TEXT("Keyboard"));
		INPUT_KIND_STRING(GameInputKindMouse, TEXT("Mouse"));
#if UE_GAMEINPUT_SUPPORTS_SENSORS
		INPUT_KIND_STRING(GameInputKindSensors, TEXT("Sensors"));
#endif
#if UE_GAMEINPUT_SUPPORTS_TOUCH
		INPUT_KIND_STRING(GameInputKindTouch, TEXT("Touch"));
		INPUT_KIND_STRING(GameInputKindMotion, TEXT("Motion"));
#endif
		INPUT_KIND_STRING(GameInputKindArcadeStick, TEXT("ArcadeStick"));
		INPUT_KIND_STRING(GameInputKindFlightStick, TEXT("FlightStick"));
		INPUT_KIND_STRING(GameInputKindGamepad, TEXT("Gamepad"));
		INPUT_KIND_STRING(GameInputKindRacingWheel, TEXT("RacingWheel"));
#if GAMEINPUT_API_VERSION < 3
		INPUT_KIND_STRING(GameInputKindUiNavigation, TEXT("UiNavigation"));
#endif
#undef INPUT_KIND_STRING

		Result.RemoveFromEnd(TEXT("|"));
		return Result;
	}

	FString LexToString(GameInputSwitchPosition SwitchPos)
	{
		switch(SwitchPos)
		{
		case GameInputSwitchCenter: return TEXT("SwitchCenter");
		case GameInputSwitchUp: return TEXT("SwitchUp");
		case GameInputSwitchUpRight: return TEXT("SwitchUpRight");
		case GameInputSwitchRight: return TEXT("SwitchRight");
		case GameInputSwitchDownRight: return TEXT("SwitchDownRight");
		case GameInputSwitchDown: return TEXT("SwitchDown");
		case GameInputSwitchDownLeft: return TEXT("SwitchDownLeft");
		case GameInputSwitchLeft: return TEXT("SwitchLeft");
		case GameInputSwitchUpLeft: return TEXT("SwitchUpLeft");
		}

		return TEXT("Unknown Switch Position");
	}

	FString LexToString(const APP_LOCAL_DEVICE_ID& DeviceId)
	{
		return BytesToHex(DeviceId.value, APP_LOCAL_DEVICE_ID_SIZE);
	}

#if UE_GAMEINPUT_SUPPORTS_SENSORS
	FString LexToString(GameInputSensorsKind InputKind)
	{
		if (InputKind == GameInputSensorsNone)
		{
			return TEXT("None");
		}

		FString Result = TEXT("");
#define INPUT_KIND_STRING(StatusFlag, DisplayName) if( InputKind & StatusFlag ) Result += (FString(DisplayName) + TEXT("|"));
		INPUT_KIND_STRING(GameInputSensorsAccelerometer, TEXT("Accelerometer"));
		INPUT_KIND_STRING(GameInputSensorsGyrometer, TEXT("Gyrometer"));
		INPUT_KIND_STRING(GameInputSensorsCompass, TEXT("Compass"));
		INPUT_KIND_STRING(GameInputSensorsOrientation, TEXT("Orientation"));
#undef INPUT_KIND_STRING

		Result.RemoveFromEnd(TEXT("|"));
		return Result;
	}

	FString LexToString(GameInputSensorAccuracy InputKind)
	{
		if (InputKind == GameInputSensorAccuracyUnknown)
		{
			return TEXT("Unknown");
		}

		FString Result = TEXT("");
#define INPUT_KIND_STRING(StatusFlag, DisplayName) if( InputKind & StatusFlag ) Result += (FString(DisplayName) + TEXT("|"));
		INPUT_KIND_STRING(GameInputSensorAccuracyUnreliable, TEXT("Unreliable"));
		INPUT_KIND_STRING(GameInputSensorAccuracyApproximate, TEXT("Approximate"));
		INPUT_KIND_STRING(GameInputSensorAccuracyHigh, TEXT("High"));
#undef INPUT_KIND_STRING

		Result.RemoveFromEnd(TEXT("|"));
		return Result;
	}
#endif	// #if UE_GAMEINPUT_SUPPORTS_SENSORS
	
	EInputDeviceConnectionState DeviceStateToConnectionState(GameInputDeviceStatus InCurrentStatus, GameInputDeviceStatus InPreviousStatus)
	{
		InCurrentStatus = InCurrentStatus & GameInputDeviceConnected;
		InPreviousStatus = InPreviousStatus & GameInputDeviceConnected;

		// If the current status has changed to connected, or the previous and current status are connected, then respond as such
		// The current and previous state may both respond as connected upon Application boot
		if ((InCurrentStatus > InPreviousStatus) || (InCurrentStatus && InPreviousStatus))
		{
			return EInputDeviceConnectionState::Connected;
		}
		else if (InPreviousStatus > InCurrentStatus)
		{
			return EInputDeviceConnectionState::Disconnected;
		}
		else
		{
			// Returned in the case of there being no connection change
			return EInputDeviceConnectionState::Invalid;
		}
	}

	const TCHAR* GetMouseButtonName(EMouseButtons::Type MouseButton)
	{
		switch (MouseButton)
		{
		case EMouseButtons::Left: return TEXT("Left");
		case EMouseButtons::Right: return TEXT("Right");
		case EMouseButtons::Middle: return TEXT("Middle");
		case EMouseButtons::Thumb01: return TEXT("Thumb01");
		case EMouseButtons::Thumb02: return TEXT("Thumb02");
		}
		return TEXT("Invalid");
	}

	bool GameInputButtonToUnrealName(const TMap<uint32, FGamepadKeyNames::Type>& UEButtonMap, uint32 ButtonMask, OUT FGamepadKeyNames::Type& OutKeyName)
	{
		if (const FGamepadKeyNames::Type* Name = UEButtonMap.Find(ButtonMask))
		{
			OutKeyName = *Name;
			return true;
		}

		return false;
	}

	const TMap<GameInputSwitchPosition, TArray<FGamepadKeyNames::Type>>& GetSwitchButtonMap()
	{
		static const TMap<GameInputSwitchPosition, TArray<FGamepadKeyNames::Type>> SwitchButtonMap
		{
			{ GameInputSwitchCenter,			{ } },
			{ GameInputSwitchUp,				{ FGamepadKeyNames::DPadUp } },
			{ GameInputSwitchUpRight,			{ FGamepadKeyNames::DPadUp, FGamepadKeyNames::DPadRight } },
			{ GameInputSwitchRight,				{ FGamepadKeyNames::DPadRight } },
			{ GameInputSwitchDownRight,			{ FGamepadKeyNames::DPadDown, FGamepadKeyNames::DPadRight } },
			{ GameInputSwitchDown,				{ FGamepadKeyNames::DPadDown } },
			{ GameInputSwitchDownLeft,			{ FGamepadKeyNames::DPadDown, FGamepadKeyNames::DPadLeft } },
			{ GameInputSwitchLeft,				{ FGamepadKeyNames::DPadLeft } },
			{ GameInputSwitchUpLeft,			{ FGamepadKeyNames::DPadUp, FGamepadKeyNames::DPadLeft } },
		};
		return SwitchButtonMap;
	}

	const TArray<FGamepadKeyNames::Type>* SwitchPositionToUnrealName(const GameInputSwitchPosition ButtonMask)
	{
		return GetSwitchButtonMap().Find(ButtonMask);
	}

	const GameInputDeviceInfo* GetDeviceInfo(IGameInputDevice* Device)
	{
		if (!Device)
		{
			return nullptr;
		}

		const GameInputDeviceInfo* Info = nullptr;
		
#if GAMEINPUT_API_VERSION >= 1
		Device->GetDeviceInfo(&Info);
#else
		Info = Device->GetDeviceInfo();
#endif

		return Info;
	}

	APP_LOCAL_DEVICE_ID GetDeviceAppLocalDeviceId(IGameInputDevice* Device)
	{
		const GameInputDeviceInfo* DeviceInfo = GetDeviceInfo(Device);

		if (!DeviceInfo)
		{
			return APP_LOCAL_DEVICE_ID{};
		}

		return DeviceInfo->deviceId;
	}

	FName GetDeviceScopeClassName()
	{
		static const FName ScopeClassName(TEXT("GameInput"));
		return ScopeClassName;
	}

	FStringView GetHardwareDeviceIdentifierName(IGameInputDevice* Device)
	{
		static const TCHAR* ID_Virtual = TEXT("Virtual");
		static const TCHAR* ID_Aggregate = TEXT("Aggregate");
		static const TCHAR* ID_XboxOne = TEXT("XboxOne");
		static const TCHAR* ID_Xbox360 = TEXT("Xbox360");
		static const TCHAR* ID_Hid = TEXT("Hid");
		static const TCHAR* ID_I8042 = TEXT("I8042");

		if (Device)
		{
			if (const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device))
			{
				// Check for any device specific overrides that may be there for custom devices
				if (const UGameInputDeveloperSettings* Settings = GetDefault<UGameInputDeveloperSettings>())
				{
					if (const FGameInputDeviceConfiguration* Config = Settings->FindDeviceConfiguration(Info))
					{
						if (Config->bOverrideHardwareDeviceIdString)
						{
							return *Config->OverriddenHardwareDeviceId;
						}					
					}
				}

				switch (Info->deviceFamily)
				{
				case GameInputFamilyVirtual:
					return ID_Virtual;
				case GameInputFamilyAggregate:
					return ID_Aggregate;
				case GameInputFamilyXbox360:
					return ID_Xbox360;
				case GameInputFamilyHid:
					return ID_Hid;
				case GameInputFamilyI8042:
					return ID_I8042;
				case GameInputFamilyXboxOne:
				default:
					return ID_XboxOne;
				}
			}
		}
				
		// If for some reason we are given a null device, default to just "Xbox One"
		return ID_XboxOne;
	}

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	FGameInputHapticEndpointFactory* GetHapticEndpointFactory()
	{
		return FGameInputBaseModule::Get().GetHapticFactory();
	}
#endif

}

#endif	// #if GAME_INPUT_SUPPORT
