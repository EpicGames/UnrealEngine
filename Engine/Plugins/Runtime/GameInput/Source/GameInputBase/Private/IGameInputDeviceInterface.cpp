// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGameInputDeviceInterface.h"
#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
#include "GameInputHapticEndpointFactory.h"
#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Misc/CoreDelegates.h"
#include "GenericPlatform/IInputInterface.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"

#if GAME_INPUT_SUPPORT
namespace UE::GameInput
{
	static TAutoConsoleVariable<bool> CVarGameInputEnumerateDeviceTypeOnConnection
	(
		TEXT("gameinput.bEnumerateDeviceTypeOnConnection"),
		true,
		TEXT("If true, EnumerateCurrentlyConnectedDeviceTypes will be called upon device connect and disconnect"),
		ECVF_Default
	);

	static TAutoConsoleVariable<bool> CVarGameInputSkipInitIfRenderDocAttached
	(
		TEXT("gameinput.bSkipInitIfRenderDocAttached"),
		true,
		TEXT("If true, GameInput device callbacks will not be registered when the RenderDoc plugin is active. "
			 "The RenderDoc plugin attempts to load the same DLLs as some of GameInput, which causes an internal abort in GameInputRedist.dll's dispatcher thread."
			 "This issue is for GameInputRedist.dll versions 3.2+"),
		ECVF_Default
	);
	
	static bool ShouldSkipFromRenderDoc()
	{
		if (!UE::GameInput::CVarGameInputSkipInitIfRenderDocAttached.GetValueOnAnyThread())
		{
			return false;
		}

		// -attachrenderdoc on the command line triggers RenderDoc to attach at startup
		if (FParse::Param(FCommandLine::Get(), TEXT("attachrenderdoc")))
		{
			return true;
		}

		// renderdoc.AutoAttach non-zero also causes RenderDoc to attach
		if (const IConsoleVariable* AutoAttachCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("renderdoc.AutoAttach")))
		{
			if (AutoAttachCVar->GetInt() != 0)
			{
				return true;
			}
		}

		return false;
	}
	
};
#endif	// GAME_INPUT_SUPPORT

IGameInputDeviceInterface::IGameInputDeviceInterface(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, IGameInput* InGameInput)
#if GAME_INPUT_SUPPORT
	: MessageHandler(InMessageHandler)
	, GameInput(InGameInput)
	, ConnectionChangeCallbackToken{}
	, CurrentlyConnectedDeviceTypes(GameInputKindUnknown)
	, bWasinitialized(false)
	, bIsAppCurrentlyConstrained(false)
	, bWasAppConstrainedLastTick(false)
#endif	// GAME_INPUT_SUPPORT
{
}

IGameInputDeviceInterface::~IGameInputDeviceInterface()
{
#if GAME_INPUT_SUPPORT
	if (ConnectionChangeCallbackToken)
	{
		if (GameInput)
		{
#if GAMEINPUT_API_VERSION >= 1
			GameInput->UnregisterCallback(ConnectionChangeCallbackToken);
#else
			GameInput->UnregisterCallback(ConnectionChangeCallbackToken, UINT64_MAX);
#endif
			
			GameInput.Reset();
		}

#if GAMEINPUT_API_VERSION >= 1
		ConnectionChangeCallbackToken = {};
#else
		ConnectionChangeCallbackToken = GAMEINPUT_INVALID_CALLBACK_TOKEN_VALUE;
#endif
	}

	FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
#endif	// GAME_INPUT_SUPPORT
}

void IGameInputDeviceInterface::Initialize()
{
#if GAME_INPUT_SUPPORT
	BindDeviceStatusCallbacks();
	bWasinitialized = true;
#endif	// GAME_INPUT_SUPPORT
}

void IGameInputDeviceInterface::Tick(float DeltaTime)
{
#if GAME_INPUT_SUPPORT

	FScopeLock Lock(&DeviceInfoCS);

	// At this point we normally will have a valid game input object.
	// However, during boot it may be still getting created in the background, 
	// so early exit in that case.
	if (!GameInput)
	{
		return;
	}
	
	// Process any input devices so long as we are not constrained
	if (!bIsAppCurrentlyConstrained)
	{
		// Handle any device connection/disconnection state changes first before attempting to process any devices
		ProcessDeferredDeviceConnectionChanges();
	}
	
#endif // GAME_INPUT_SUPPORT
}

void IGameInputDeviceInterface::SendControllerEvents()
{
#if GAME_INPUT_SUPPORT

	// At this point we normally will have a valid game input object.
	// However, during boot it may be still getting created in the background, 
	// so early exit in that case.
	
	if (!bWasinitialized)
	{
		return;
	}
	
	if (!GameInput)
	{
		return;
	}

	FScopeLock Lock(&DeviceInfoCS);

	// On the first update coming back from being constrained, we want to reset
	// the state of inputs
	if (bIsAppCurrentlyConstrained && !bWasAppConstrainedLastTick)
	{
		DetermineStateAfterFirstUnconstrainedUpdate();
	}

	// Process any input devices so long as we are not constrained
	if (!bIsAppCurrentlyConstrained)
	{
		// A map of Platform users to a bitmask of any reading kinds that were processed this frame.
		// This is used by the devices to keep track of which users had which readings, and use that
		// state to determine if we can process a given reading.
		TMap<FPlatformUserId, GameInputKind> PlatformUsersWhoHaveHadInputThisFrame;

		// The allowed input kinds that we can read from the IGameInput interface.
		const GameInputKind AllowedInputKindsThisFrame = GetCurrentGameInputKindSupport();

		for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
		{
			const FPlatformUserId UserId = KnownDevice->GetPlatformUserId();

			// Figure out what GameInputKind's this platform user has already handled this frame.
			GameInputKind& AlreadyProcessedInputKinds = PlatformUsersWhoHaveHadInputThisFrame.FindOrAdd(UserId, /* default init value */ GameInputKindUnknown);

			// Actually process our input here, and get a bitmask back of what GameInputKind's have sent events.
			const GameInputKind InputKindsWithEvents = KnownDevice->ProcessInput(GameInput, AllowedInputKindsThisFrame, AlreadyProcessedInputKinds);

			// Keep track what kinds of input that this platform user has processed this frame for use on the next device
			AlreadyProcessedInputKinds |= InputKindsWithEvents;
			
			// Keep track of the most recent device that is being used by each given platform user
			// We need to do this whenever we receive input, which is true as long as there was an input kind processed this frame.
			if (InputKindsWithEvents != GameInputKindUnknown)
			{
				// If we know input came from a device associated to this platform user already, then check our timestamp to see if it is newer then it
				if (const TSharedPtr<FGameInputDeviceContainer>* MostRecentKnownDevice = PlatformUserIdToMostRecentDeviceContainer.Find(UserId))
				{
					// If the current device that was just processed has a more recent timestamp then the known one, then
					// use it as our most recent device instead
					if (KnownDevice->GetLastReadingTimestamp() > (*MostRecentKnownDevice)->GetLastReadingTimestamp())
					{
						PlatformUserIdToMostRecentDeviceContainer[UserId] = KnownDevice;
					}
				}
				else
				{
					PlatformUserIdToMostRecentDeviceContainer.Add(UserId, KnownDevice);
				}
			}
		}
	}

	bWasAppConstrainedLastTick = bIsAppCurrentlyConstrained;

#endif// GAME_INPUT_SUPPORT
}

void IGameInputDeviceInterface::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
#if GAME_INPUT_SUPPORT
	MessageHandler = InMessageHandler;

	for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		KnownDevice->SetMessageHandler(InMessageHandler);
	}
#endif	// GAME_INPUT_SUPPORT
}

bool IGameInputDeviceInterface::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// required by IInputDevice interface
	return false;
}

void IGameInputDeviceInterface::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
#if GAME_INPUT_SUPPORT
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FPlatformUserId UserId = PLATFORMUSERID_NONE;
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	DeviceMapper.RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);

	if (!UserId.IsValid())
	{
		return;
	}

	GameInputRumbleParams RumbleParams = {};

	switch (ChannelType)
	{
	case FForceFeedbackChannelType::LEFT_LARGE:
		RumbleParams.lowFrequency = FMath::Clamp(Value, 0.0f, 1.0f);
		break;
	case FForceFeedbackChannelType::LEFT_SMALL:
		RumbleParams.leftTrigger = FMath::Clamp(Value, 0.0f, 1.0f);
		break;
	case FForceFeedbackChannelType::RIGHT_LARGE:
		RumbleParams.highFrequency = FMath::Clamp(Value, 0.0f, 1.0f);
		break;
	case FForceFeedbackChannelType::RIGHT_SMALL:
		RumbleParams.rightTrigger = FMath::Clamp(Value, 0.0f, 1.0f);
		break;
	}

	// Send a rumble event for every input device that is mapped to this platform user
	for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		if (DeviceMapper.GetUserForInputDevice(KnownDevice->GetDeviceId()) == UserId)
		{
			if (IGameInputDevice* Device = KnownDevice->GetGameInputDevice())
			{
				Device->SetRumbleState(&RumbleParams);
			}
		}
	}

#endif	// #if GAME_INPUT_SUPPORT
}

void IGameInputDeviceInterface::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
{
#if GAME_INPUT_SUPPORT
	// TODO: Allow native input device id's to FInputDeviceId's in the platform input device mapper
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FPlatformUserId UserId = PLATFORMUSERID_NONE;
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	DeviceMapper.RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);

	if (!UserId.IsValid())
	{
		return;
	}

	GameInputRumbleParams RumbleParams = {};

	RumbleParams.lowFrequency = FMath::Clamp(Values.LeftLarge, 0.0f, 1.0f);
	RumbleParams.leftTrigger = FMath::Clamp(Values.LeftSmall, 0.0f, 1.0f);
	RumbleParams.highFrequency = FMath::Clamp(Values.RightLarge, 0.0f, 1.0f);
	RumbleParams.rightTrigger = FMath::Clamp(Values.RightSmall, 0.0f, 1.0f);

	// Send a rumble event for every input device that is mapped to this platform user
	for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		if (DeviceMapper.GetUserForInputDevice(KnownDevice->GetDeviceId()) == UserId)
		{
			if (IGameInputDevice* Device = KnownDevice->GetGameInputDevice())
			{
				Device->SetRumbleState(&RumbleParams);
			}
		}
	}
#endif	// #if GAME_INPUT_SUPPORT
}

bool IGameInputDeviceInterface::IsGamepadAttached() const
{
#if GAME_INPUT_SUPPORT
	// We will treat both Gamepads and Controller's as "Gamepads" as far as UE is concerned. 
	return CurrentlyConnectedDeviceTypes & GameInputKindGamepad || CurrentlyConnectedDeviceTypes & GameInputKindController;
#else
	return false;
#endif	// GAME_INPUT_SUPPORT
}

#if GAME_INPUT_SUPPORT

GameInputKind IGameInputDeviceInterface::GetCurrentGameInputKindSupport() const
{
	GameInputKind RegisterInputKindMask = GameInputKindUnknown;
	
	const UGameInputPlatformSettings* PlatformSettings = UGameInputPlatformSettings::Get();
	
	if (PlatformSettings->bProcessController)
	{
		RegisterInputKindMask |= (GameInputKindController | GameInputKindControllerAxis | GameInputKindControllerButton | GameInputKindControllerSwitch);
	}

#if UE_GAMEINPUT_SUPPORTS_RAW
	if (PlatformSettings->bProcessRawInput)
	{
		RegisterInputKindMask |= GameInputKindRawDeviceReport;
	}
#endif

	if (PlatformSettings->bProcessGamepad)
	{
		RegisterInputKindMask |= GameInputKindGamepad;
	}

#if UE_GAMEINPUT_SUPPORTS_SENSORS
	if (PlatformSettings->bProcessSensors)
	{
		RegisterInputKindMask |= GameInputKindSensors;
	}
#endif
	
	if (PlatformSettings->bProcessKeyboard)
	{
		RegisterInputKindMask |= GameInputKindKeyboard;
	}

	if (PlatformSettings->bProcessMouse)
	{
		RegisterInputKindMask |= GameInputKindMouse;
	}

	if (PlatformSettings->bProcessRacingWheel)
	{
		RegisterInputKindMask |= GameInputKindRacingWheel;
	}

	if (PlatformSettings->bProcessArcadeStick)
	{
		RegisterInputKindMask |= GameInputKindArcadeStick;
	}

	if (PlatformSettings->bProcessFlightStick)
	{
		RegisterInputKindMask |= GameInputKindFlightStick;
	}

	return RegisterInputKindMask;
}

bool IGameInputDeviceInterface::BindDeviceStatusCallbacks()
{
	if (!GameInput)
	{
		return false;
	}
	
	// First, check if we are already bound to something. If so, unbind it
	if (ConnectionChangeCallbackToken)
	{
#if GAMEINPUT_API_VERSION >= 1
		GameInput->UnregisterCallback(ConnectionChangeCallbackToken);
		ConnectionChangeCallbackToken = {};
#else
		GameInput->UnregisterCallback(ConnectionChangeCallbackToken, UINT64_MAX);
		ConnectionChangeCallbackToken = GAMEINPUT_INVALID_CALLBACK_TOKEN_VALUE;
#endif
	}

	// Hook up device callbacks 
	auto DeviceCallbackFn = [](GameInputCallbackToken CallbackToken, void* Context, IGameInputDevice* Device, uint64 Timestamp, GameInputDeviceStatus CurrentStatus, GameInputDeviceStatus PreviousStatus)
	{
		static_cast<IGameInputDeviceInterface*>(Context)->OnDeviceConnectionChanged(CallbackToken, Device, Timestamp, CurrentStatus, PreviousStatus);
	};

	// TODO: should this be a member variable that subclasses can override? That might be desirable on PC
	GameInputDeviceStatus DeviceStatusMask = (GameInputDeviceNoStatus | GameInputDeviceConnected);

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	if (GetDefault<UGameInputDeveloperSettings>()->IsHapticSupportEnabled())
	{
		DeviceStatusMask |= GameInputDeviceHapticInfoReady;	
	}
#endif
	const GameInputKind RegisterInputKindMask = GetCurrentGameInputKindSupport();

	UE_LOGF(LogGameInput, Log, "Registering Device Callback for GameInputKind: '%ls'. Listening for Device Status: '%ls'.", *UE::GameInput::LexToString(RegisterInputKindMask), *UE::GameInput::LexToString(DeviceStatusMask));
	
	// There is a crash in the internals of the GameInput library when render doc attaches,
	// So for now, avoid attempting to bind because it will crash the editor. 
	if (UE::GameInput::ShouldSkipFromRenderDoc())
	{
		UE_LOGF(LogGameInput, Warning, "RenderDoc attach detected (-attachrenderdoc on the command line, or renderdoc.AutoAttach is non-zero). Skipping GameInput device callback registration to avoid a known crash in GameInputRedist.dll (toggle 'gameinput.bSkipInitIfRenderDocAttached' to disable this and proceed anyway.).");
		return false;
	}

	GameInput->RegisterDeviceCallback(nullptr, RegisterInputKindMask, DeviceStatusMask, GameInputBlockingEnumeration, this, DeviceCallbackFn, &ConnectionChangeCallbackToken);

	// get notified when the app is constrained & unconstrained
	FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(this, &IGameInputDeviceInterface::OnAppConstrained);
	FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &IGameInputDeviceInterface::OnAppUnconstrained);

	// We are successful if the callback token is valid
#if GAMEINPUT_API_VERSION >= 1
	return true;
#else
	return ConnectionChangeCallbackToken != GAMEINPUT_INVALID_CALLBACK_TOKEN_VALUE;
#endif
}

void IGameInputDeviceInterface::OnDeviceConnectionChanged(GameInputCallbackToken CallbackToken, IGameInputDevice* Device, uint64 Timestamp, GameInputDeviceStatus CurrentStatus, GameInputDeviceStatus PreviousStatus)
{	
	if (!IsSupportedDevice(Device))
	{
		UE_LOGF(
			LogGameInput,
			Verbose,
			"%ls is an unsupported Device and will not be processed. (Connection Changed detected from '%ls' to '%ls')",
			*UE::GameInput::LexToString(Device), *UE::GameInput::LexToString(PreviousStatus), *UE::GameInput::LexToString(CurrentStatus));
		
		return;
	}
	
	FScopeLock Lock(&DeviceInfoCS);
	
	// This event may come in async from GameInput from outside the game thread, so defer it until Tick so we can guarantee game thread access
	IGameInputDeviceInterface::FDeferredDeviceConnectionChanges Event = {};
	Event.Device = Device;
	Event.Timestamp = Timestamp;
	Event.Status = UE::GameInput::DeviceStateToConnectionState(CurrentStatus, PreviousStatus);
	Event.CurrentStatus = CurrentStatus;
	Event.PreviousStatus = PreviousStatus;

	DeferredDeviceConnectionChanges.Emplace(Event);

	UE_LOGF(LogGameInput, Log, "%ls : Device Connection Changed from '%ls' to '%ls', deferring event processing to next game thread tick",
		*UE::GameInput::LexToString(Device) ,*UE::GameInput::LexToString(PreviousStatus), *UE::GameInput::LexToString(CurrentStatus));
}

bool IGameInputDeviceInterface::IsSupportedDevice(IGameInputDevice* Device) const
{
	if (!Device)
	{
		return false;
	}
	
	const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);
	
	if (!Info)
	{
		return false;
	}

	const UGameInputDeveloperSettings* Settings = GetDefault<UGameInputDeveloperSettings>();

	if (Settings && Settings->ShouldIgnoreDevice(Info))
	{
		return false;
	}
	
	return true;
}

void IGameInputDeviceInterface::HandleHapticsReady(IGameInputDevice* Device, uint64 Timestamp)
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	// TODO: Any shared haptic setup can be placed here
	GameInputHapticInfo HapticInfo = {};
	
	HRESULT Res = Device->GetHapticInfo(&HapticInfo);
	if (FAILED(Res))
	{
		UE_LOGF(LogGameInput, Error, "Failed to get haptic info for device %ls", *UE::GameInput::LexToString(Device));
		return;
	}
	
	HandleHapticsReady_Impl(Device, Timestamp, HapticInfo);
#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
}

void IGameInputDeviceInterface::HandleHapticsReady_Impl(IGameInputDevice* Device, uint64 Timestamp, GameInputHapticInfo& HapticInfo)
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	FGameInputHapticEndpointFactory* HapticFactory = UE::GameInput::GetHapticEndpointFactory();
	if (!HapticFactory)
	{
		return;
	}

	// Get the hardware device ID directly from the IGameInputDevice — this is
	// available even if the engine hasn't assigned an FInputDeviceId yet.
	const APP_LOCAL_DEVICE_ID DeviceId = UE::GameInput::GetDeviceAppLocalDeviceId(Device);

	// We may receive multiple haptics-ready notifications for the same device.
	// Skip if this device's haptic audio is already initialized.
	if (HapticFactory->IsDeviceInitialized(DeviceId))
	{
		return;
	}

	FGameInputDeviceContainer* Container = GetDeviceData(Device);
	if (!Container)
	{
		// The device container may not exist yet if the haptics-ready event arrived
		// before the connected event. We'll be called again once the container is created.
		return;
	}

	const FString AudioEndpointId(HapticInfo.audioEndpointId);
	const int32 NumLocations = FMath::Clamp((int32)HapticInfo.locationCount, 0, (int32)GAMEINPUT_HAPTIC_MAX_LOCATIONS);

	// Pass the haptic location GUID array through to the WASAPI renderer so it can
	// build the correct WAVEFORMATEXTENSIBLE channel mask. Positionally indexed:
	// locations[i] occupies output channel i.
	const TArrayView<const GUID> HapticLocations(HapticInfo.locations, NumLocations);

	UE_LOG(LogGameInput, Log,
		TEXT("HandleHapticsReady_Impl: setting up haptic audio for endpoint='%s', locations=%d"),
		*AudioEndpointId, NumLocations);

	HapticFactory->InitializeDevice(DeviceId, AudioEndpointId, HapticLocations);
#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
}

void IGameInputDeviceInterface::TeardownHapticAudio(IGameInputDevice* Device)
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	// This is a public dispatch function where, if in the future, there is some shared logic that we want to apply
	// every time haptics are torn down, we can add it here a bit easier.
	TeardownHapticAudio_Impl(Device);
#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
}

void IGameInputDeviceInterface::TeardownHapticAudio_Impl(IGameInputDevice* Device)
{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	FGameInputHapticEndpointFactory* HapticFactory = UE::GameInput::GetHapticEndpointFactory();
	if (!HapticFactory)
	{
		return;
	}

	FGameInputDeviceContainer* Container = GetDeviceData(Device);
	if (!Container)
	{
		return;
	}

	HapticFactory->RemoveDevice(Container->GetGameInputDeviceId());
#endif	// UE_GAME_INPUT_SUPPORTS_HAPTICS
}

void IGameInputDeviceInterface::ProcessDeferredDeviceConnectionChanges()
{
	// We only want to actually handle device connection events in the game thread
	// because a lot of listeners will be in game or ui code that reference the FSlateApplication::Get function.
	check(IsInGameThread());
	if (!DeferredDeviceConnectionChanges.IsEmpty())
	{
		// A bit of pre-processing of the connection events to ensure a stable order of execution
		CoalesceDeferredDeviceConnectionChanges();
		
		for (const FDeferredDeviceConnectionChanges& Event : DeferredDeviceConnectionChanges)
		{
#if UE_GAME_INPUT_SUPPORTS_HAPTICS
			// If this event was for a connection, we also need to check if there is Haptic info ready yet.
			// We may receive an event after the initial connection from the device to handle haptics being ready.
			const bool bCurrentlyHasHapticsReady = (Event.CurrentStatus & GameInputDeviceStatus::GameInputDeviceHapticInfoReady) != 0;
			const bool bHadHapticsReady = (Event.PreviousStatus & GameInputDeviceStatus::GameInputDeviceHapticInfoReady) != 0;

			// If the haptics were previously ready, and they no longer are, then we need to tear them down.
			// This can happen when the device is connected or disconnected
			if (bHadHapticsReady && !bCurrentlyHasHapticsReady)
			{
				TeardownHapticAudio(Event.Device);
			}
#endif	// #if UE_GAME_INPUT_SUPPORTS_HAPTICS
			
			// No status means that the device has been disconnected
			if (Event.Status == EInputDeviceConnectionState::Disconnected)
			{
				HandleDeviceDisconnected(Event.Device, Event.Timestamp);
			}
			else if (Event.Status == EInputDeviceConnectionState::Connected)
			{
				// It is possible that we receive multiple connection events for devices with haptic support
				// on the same frame. In that case, we may have already run the actual device connection logic
				// from a previous event. We can detect these by checking the actual game input connection status
				// and ensuring that we only actually do any new platform user mapping when we need to.
				// TLDR: This prevents us from "double connecting" if we receive multiple connect events per frame
				if (!(Event.PreviousStatus & GameInputDeviceStatus::GameInputDeviceConnected))
				{
					HandleDeviceConnected(Event.Device, Event.Timestamp);
				}
			}

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
			// If the haptics are ready, and they weren't before, then we need to init them for a connected device.
			if (bCurrentlyHasHapticsReady && !bHadHapticsReady)
			{
				HandleHapticsReady(Event.Device, Event.Timestamp);
			}
#endif
		}
		
		DeferredDeviceConnectionChanges.Reset();
	}
}

void IGameInputDeviceInterface::CoalesceDeferredDeviceConnectionChanges()
{
	// Sort the connection events in their reported timestamp order from GameInput, which may not be the order which we received
	// them in on the game thread. Process the oldest timestamps first.
	DeferredDeviceConnectionChanges.StableSort(
		[](const FDeferredDeviceConnectionChanges& A, const FDeferredDeviceConnectionChanges& B)
			{
				return A.Timestamp < B.Timestamp;
			});
}

void IGameInputDeviceInterface::OnAppConstrained()
{
	bIsAppCurrentlyConstrained = true;
}

void IGameInputDeviceInterface::OnAppUnconstrained()
{
	bIsAppCurrentlyConstrained = false;
}

void IGameInputDeviceInterface::DetermineStateAfterFirstUnconstrainedUpdate()
{
	// This should only be called on the first update after coming back from being constrained
	ensure(bIsAppCurrentlyConstrained && !bWasAppConstrainedLastTick);

	for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		KnownDevice->ClearInputState(GameInput);
	}	
}

FGameInputDeviceContainer* IGameInputDeviceInterface::GetDeviceData(IGameInputDevice* InDevice)
{
	// Check if we already know about the given device
	if (!InDevice)
	{
		return nullptr;
	}

	// Check if we have seen this device's APP_LOCAL_DEVICE_ID before. The IGameInputDevice* could have been invalidated if the device 
	// is being re-connected, but it will have the APP_LOCAL_DEVICE_ID would be the same.
	const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(InDevice);

	FGameInputDeviceContainer* RetVal = nullptr;

	for (const TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		const APP_LOCAL_DEVICE_ID KnownDeviceId = KnownDevice->GetGameInputDeviceId();

		if (KnownDevice->GetGameInputDevice() == InDevice ||
			(Info && FMemory::Memcmp(&Info->deviceId, &KnownDeviceId, sizeof(KnownDeviceId)) == 0))
		{
			RetVal = KnownDevice.Get();
			break;
		}
	}


	return RetVal;
}

FGameInputDeviceContainer* IGameInputDeviceInterface::GetOrCreateDeviceData(IGameInputDevice* InDevice)
{
	// Check if we already know about the given device
	if (!InDevice)
	{
		return nullptr;
	}

	// Check if we already have some device data about this...
	if (FGameInputDeviceContainer* ExistingDevice = GetDeviceData(InDevice))
	{
		// For existing devices, we want to ensure that their IGameInputDevice pointer matches up with what was given.
		// This may be the case if you disconnect and then reconnect a device, because we can still find it's associated
		// FGameInputDeviceContainer based on the  APP_LOCAL_DEVICE_ID, but the IGameInputDevice pointer would be null.
		ExistingDevice->SetGameInputDevice(InDevice);

		return ExistingDevice;
	}

	// ... if not, then create a new one.
	return CreateDeviceData(InDevice);
}

void IGameInputDeviceInterface::EnumerateCurrentlyConnectedDeviceTypes()
{
	if (!UE::GameInput::CVarGameInputEnumerateDeviceTypeOnConnection.GetValueOnAnyThread())
	{
		return;
	}

	CurrentlyConnectedDeviceTypes = GameInputKindUnknown;

	// Check all of our devices and their supported input flags
	for (TSharedPtr<FGameInputDeviceContainer>& KnownDevice : DeviceData)
	{
		if (KnownDevice)
		{
			if (IGameInputDevice* Device = KnownDevice->GetGameInputDevice())
			{
				if (const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device))				
				{
					CurrentlyConnectedDeviceTypes |= Info->supportedInput;
				}
			}
		}
	}
}

#endif	// GAME_INPUT_SUPPORT
