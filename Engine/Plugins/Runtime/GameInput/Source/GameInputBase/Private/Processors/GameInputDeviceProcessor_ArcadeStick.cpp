// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_ArcadeStick.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputKeyTypes.h"

#if GAME_INPUT_SUPPORT

namespace UE::GameInput
{
	/** A map of uint32 GameInput button bitmask flags to the associated Unreal Engine FKey name. */
	static const TMap<uint32, FGamepadKeyNames::Type>& GetArcadeStickButtonMap()
	{
		static const TMap<uint32, FGamepadKeyNames::Type> GamepadButtonMap
		{
			// Generic gamepad buttons
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickNone), FGamepadKeyNames::Invalid},
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickMenu), FGamepadKeyNames::SpecialRight },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickView), FGamepadKeyNames::SpecialLeft },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickUp), FGamepadKeyNames::DPadUp },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickDown), FGamepadKeyNames::DPadDown },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickLeft), FGamepadKeyNames::DPadLeft },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickRight), FGamepadKeyNames::DPadRight },

			// Unique to arcade sticks
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction1), FGameInputKeys::ArcadeStick_Action1.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction2), FGameInputKeys::ArcadeStick_Action2.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction3), FGameInputKeys::ArcadeStick_Action3.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction4), FGameInputKeys::ArcadeStick_Action4.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction5), FGameInputKeys::ArcadeStick_Action5.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickAction6), FGameInputKeys::ArcadeStick_Action6.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickSpecial1), FGameInputKeys::ArcadeStick_Special1.GetFName() },
			{ static_cast<uint32>(GameInputArcadeStickButtons::GameInputArcadeStickSpecial2), FGameInputKeys::ArcadeStick_Special2.GetFName() }
		};
		return GamepadButtonMap;
	}
};


FGameInputArcadeStickProcessor::FGameInputArcadeStickProcessor()
	: IGameInputDeviceProcessor()
{
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
}

bool FGameInputArcadeStickProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	// Can't do anything for an invalid platform user or no reading.
	if (!Params.PlatformUserId.IsValid() || !Params.Reading)
	{
		return bRes;
	}

	GameInputArcadeStickState StickState;
	if (!Params.Reading->GetArcadeStickState(&StickState))
	{
		return bRes;
	}

	const uint32 CurrentButtonHeldMask = static_cast<uint32>(StickState.buttons);
	uint32 LastButtonHeldMask = static_cast<uint32>(PreviousState.buttons);

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		OUT LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetArcadeStickButtonMap(),
		MaxSupportedButtons);

	// Keep track of the button state so that we can compare it next time it is processed
	PreviousState = StickState;

	return true;
}

void FGameInputArcadeStickProcessor::ClearState(const FGameInputEventParams& Params)
{	
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return;
	}

	// Evaluate the buttons as if none have been pressed (i.e. the button mask is 0)
	constexpr uint32 CurrentButtonHeldMask = 0x00;
	uint32 LastButtonHeldMask = static_cast<uint32>(PreviousState.buttons);

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		OUT LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetArcadeStickButtonMap(),
		MaxSupportedButtons);

	// Zero out the repeat and previous states
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
}

GameInputKind FGameInputArcadeStickProcessor::GetSupportedReadingKind() const
{
	return GameInputKindArcadeStick;
}

#endif	// GAME_INPUT_SUPPORT