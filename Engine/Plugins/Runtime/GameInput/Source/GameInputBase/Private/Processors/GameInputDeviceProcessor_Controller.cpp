// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_Controller.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "InputCoreTypes.h"

#if GAME_INPUT_SUPPORT

/**
* Returns a map of GameInputLabel's to their associated Unreal Engine FKey names.
*/
const TMap<GameInputLabel, FGamepadKeyNames::Type>& FGameInputControllerDeviceProcessor::GetGameInputButtonLabelToUnrealName()
{
	static const TMap<GameInputLabel, FGamepadKeyNames::Type> LabelMap
	{
		{ GameInputLabelUnknown, FGamepadKeyNames::Invalid },
		{ GameInputLabelNone, FGamepadKeyNames::Invalid },
		{ GameInputLabelXboxGuide, FGamepadKeyNames::SpecialRight },	// TODO: Check if this is correct
		{ GameInputLabelXboxBack, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelXboxStart, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelXboxMenu, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelXboxView, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelXboxA, FGamepadKeyNames::FaceButtonBottom },
		{ GameInputLabelXboxB, FGamepadKeyNames::FaceButtonRight },
		{ GameInputLabelXboxX, FGamepadKeyNames::FaceButtonLeft },
		{ GameInputLabelXboxY, FGamepadKeyNames::FaceButtonTop },
		{ GameInputLabelXboxDPadUp, FGamepadKeyNames::DPadUp },
		{ GameInputLabelXboxDPadDown, FGamepadKeyNames::DPadDown },
		{ GameInputLabelXboxDPadLeft, FGamepadKeyNames::DPadLeft },
		{ GameInputLabelXboxDPadRight, FGamepadKeyNames::DPadRight },
		{ GameInputLabelXboxLeftShoulder, FGamepadKeyNames::LeftShoulder },
		{ GameInputLabelXboxLeftTrigger, FGamepadKeyNames::LeftTriggerAnalog },
		{ GameInputLabelXboxLeftStickButton, FGamepadKeyNames::LeftThumb },
		{ GameInputLabelXboxRightShoulder, FGamepadKeyNames::RightShoulder },
		{ GameInputLabelXboxRightTrigger, FGamepadKeyNames::RightTriggerAnalog },
		{ GameInputLabelXboxRightStickButton, FGamepadKeyNames::RightThumb },
		{ GameInputLabelXboxPaddle1, FGamepadKeyNames::Invalid },				// TODO: Do we need special additional FKey's for these paddle types?
		{ GameInputLabelXboxPaddle2, FGamepadKeyNames::Invalid },				// Return invalid for now, but I thought the Xbox One pro controller would
		{ GameInputLabelXboxPaddle3, FGamepadKeyNames::Invalid },				// be handled by the OS itself via virtual remapping
		{ GameInputLabelXboxPaddle4, FGamepadKeyNames::Invalid },
		{ GameInputLabelLetterA, EKeys::A.GetFName() },
		{ GameInputLabelLetterB, EKeys::B.GetFName() },
		{ GameInputLabelLetterC, EKeys::C.GetFName() },
		{ GameInputLabelLetterD, EKeys::D.GetFName() },
		{ GameInputLabelLetterE, EKeys::E.GetFName() },
		{ GameInputLabelLetterF, EKeys::F.GetFName() },
		{ GameInputLabelLetterG, EKeys::G.GetFName() },
		{ GameInputLabelLetterH, EKeys::H.GetFName() },
		{ GameInputLabelLetterI, EKeys::I.GetFName() },
		{ GameInputLabelLetterJ, EKeys::J.GetFName() },
		{ GameInputLabelLetterK, EKeys::K.GetFName() },
		{ GameInputLabelLetterL, EKeys::L.GetFName() },
		{ GameInputLabelLetterM, EKeys::M.GetFName() },
		{ GameInputLabelLetterN, EKeys::N.GetFName() },
		{ GameInputLabelLetterO, EKeys::O.GetFName() },
		{ GameInputLabelLetterP, EKeys::P.GetFName() },
		{ GameInputLabelLetterQ, EKeys::Q.GetFName() },
		{ GameInputLabelLetterR, EKeys::R.GetFName() },
		{ GameInputLabelLetterS, EKeys::S.GetFName() },
		{ GameInputLabelLetterT, EKeys::T.GetFName() },
		{ GameInputLabelLetterU, EKeys::U.GetFName() },
		{ GameInputLabelLetterV, EKeys::V.GetFName() },
		{ GameInputLabelLetterW, EKeys::W.GetFName() },
		{ GameInputLabelLetterX, EKeys::X.GetFName() },
		{ GameInputLabelLetterY, EKeys::Y.GetFName() },
		{ GameInputLabelLetterZ, EKeys::Z.GetFName() },
		{ GameInputLabelNumber0, EKeys::Zero.GetFName() },
		{ GameInputLabelNumber1, EKeys::One.GetFName() },
		{ GameInputLabelNumber2, EKeys::Two.GetFName() },
		{ GameInputLabelNumber3, EKeys::Three.GetFName() },
		{ GameInputLabelNumber4, EKeys::Four.GetFName() },
		{ GameInputLabelNumber5, EKeys::Five.GetFName() },
		{ GameInputLabelNumber6, EKeys::Six.GetFName() },
		{ GameInputLabelNumber7, EKeys::Seven.GetFName() },
		{ GameInputLabelNumber8, EKeys::Eight.GetFName() },
		{ GameInputLabelNumber9, EKeys::Nine.GetFName() },
		{ GameInputLabelArrowUp, EKeys::Up.GetFName() },
		{ GameInputLabelArrowUpRight, EKeys::Up.GetFName() },
		{ GameInputLabelArrowRight, EKeys::Right.GetFName() },
		{ GameInputLabelArrowDownRight, EKeys::Down.GetFName() },	// TODO: We should support multiple FKey's here, like we do for switches
		{ GameInputLabelArrowDown, EKeys::Down.GetFName() },
		{ GameInputLabelArrowDownLLeft, EKeys::Down.GetFName() },
		{ GameInputLabelArrowLeft, EKeys::Left.GetFName() },
		{ GameInputLabelArrowUpLeft, FGamepadKeyNames::DPadUp },
		{ GameInputLabelArrowUpDown, FGamepadKeyNames::DPadUp },
		{ GameInputLabelArrowLeftRight, FGamepadKeyNames::DPadUp },
		{ GameInputLabelArrowUpDownLeftRight, FGamepadKeyNames::DPadUp },
		{ GameInputLabelArrowClockwise, FGamepadKeyNames::DPadUp },		// TODO: new key for this          
		{ GameInputLabelArrowCounterClockwise, FGamepadKeyNames::DPadUp },	// TODO: new key for this   
		{ GameInputLabelArrowReturn, EKeys::Enter.GetFName() },
		{ GameInputLabelIconBranding, EKeys::Home.GetFName() },		// TODO: I dont think we have a UE key for this, maybe we use home?     
		{ GameInputLabelIconHome, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelIconMenu, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelIconCross, FGamepadKeyNames::FaceButtonBottom },
		{ GameInputLabelIconCircle, FGamepadKeyNames::FaceButtonRight },
		{ GameInputLabelIconSquare, FGamepadKeyNames::FaceButtonLeft },
		{ GameInputLabelIconTriangle, FGamepadKeyNames::FaceButtonTop },
		{ GameInputLabelIconStar, EKeys::Asterix.GetFName() },    // TODO: Star? Is this the asterix?            
		{ GameInputLabelIconDPadUp, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconDPadDown, FGamepadKeyNames::DPadDown },
		{ GameInputLabelIconDPadLeft, FGamepadKeyNames::DPadLeft },
		{ GameInputLabelIconDPadRight, FGamepadKeyNames::DPadRight },
		{ GameInputLabelIconDialClockwise, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconDialCounterClockwise, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconSliderLeftRight, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconSliderUpDown, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconWheelUpDown, FGamepadKeyNames::DPadUp },
		{ GameInputLabelIconPlus, EKeys::Add.GetFName() },
		{ GameInputLabelIconMinus, EKeys::Subtract.GetFName() },
		{ GameInputLabelIconSuspension, FGamepadKeyNames::DPadUp },
		{ GameInputLabelHome, EKeys::Home.GetFName() },			// TODO: Do we have a gamepad key for guide?            
		{ GameInputLabelGuide, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelMode, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelSelect, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelMenu, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelView, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelBack, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelStart, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelOptions, FGamepadKeyNames::SpecialRight },
		{ GameInputLabelShare, FGamepadKeyNames::SpecialLeft },
		{ GameInputLabelUp, FGamepadKeyNames::DPadUp },
		{ GameInputLabelDown, FGamepadKeyNames::DPadDown },
		{ GameInputLabelLeft, FGamepadKeyNames::DPadLeft },
		{ GameInputLabelRight, FGamepadKeyNames::DPadRight },
		{ GameInputLabelLB, FGamepadKeyNames::LeftShoulder },
		{ GameInputLabelLT, FGamepadKeyNames::LeftTriggerAnalog },
		{ GameInputLabelLSB, FGamepadKeyNames::LeftShoulder },
		{ GameInputLabelL1, FGamepadKeyNames::LeftShoulder },
		{ GameInputLabelL2, FGamepadKeyNames::LeftTriggerAnalog },
		{ GameInputLabelL3, FGamepadKeyNames::DPadUp },
		{ GameInputLabelRB, FGamepadKeyNames::RightShoulder },
		{ GameInputLabelRT, FGamepadKeyNames::RightTriggerAnalog },
		{ GameInputLabelRSB, FGamepadKeyNames::RightShoulder },
		{ GameInputLabelR1, FGamepadKeyNames::RightShoulder },
		{ GameInputLabelR2, FGamepadKeyNames::RightTriggerAnalog },
		{ GameInputLabelR3, FGamepadKeyNames::DPadUp },
	};

	return LabelMap;
}

FGameInputControllerDeviceProcessor::FGameInputControllerDeviceProcessor()
	: IGameInputDeviceProcessor()
{
	// Ensure that our switch repeat times array has some default values set on it by default
	// because we access it with the [] operator when processing it. 
	SwitchRepeatTimes.AddDefaulted(static_cast<uint32>(GameInputSwitchUpLeft) + 1);
}

bool FGameInputControllerDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	const FGameInputDeviceConfiguration* ControllerConfig = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo());

	// Note that we use the current reading here. "ProcessInput" can be called multiple times per frame
	// if there is more then one input reading in the stack. We want to process all button and switch states
	// to ensure that we don't miss one
	bRes |= ProcessControllerSwitchState(Params, ControllerConfig, Params.Reading);
	bRes |= ProcessControllerButtonState(Params, ControllerConfig, Params.Reading);
	++NumReadingsProcessedThisFrame;

	return bRes;
}

bool FGameInputControllerDeviceProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	// On the last input reading for the frame, the "Current Reading" should always be null. we only care for the LastReading here
	ensure(Params.Reading == nullptr);

	bool bRes = false;

	const FGameInputDeviceConfiguration* ControllerConfig = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo());

	const bool bProcessedAnyButtonsThisFrame = (NumReadingsProcessedThisFrame > 0);

	// This is the last reading of this frame, reset the counter to 0
	NumReadingsProcessedThisFrame = 0;

	// Note that we use "Params.PreviousReading", because the current reading will be null. 
	if (!bProcessedAnyButtonsThisFrame)
	{
		bRes |= ProcessControllerSwitchState(Params, ControllerConfig, Params.PreviousReading);
		bRes |= ProcessControllerButtonState(Params, ControllerConfig, Params.PreviousReading);
	}

	bRes |= ProcessControllerAxisState(Params, ControllerConfig, Params.PreviousReading);

	return bRes;
}

void FGameInputControllerDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// Reset the switches to all be "center" positions
	{
		const uint32 SwitchCount = static_cast<uint32>(PreviousSwitchPositions.Num());

		TArray<GameInputSwitchPosition> SwitchPositions;
		SwitchPositions.AddUninitialized(SwitchCount);
		// Evaluate all switch positions
		for (uint32 i = 0; i < SwitchCount; ++i)
		{
			// The PreviousSwitchPositions will be updated to the current switch position by the evaluate function
			EvaluateSwitchState(
				Params,
				SwitchPositions[i],
				OUT PreviousSwitchPositions[i],
				SwitchRepeatTimes);
		}
	}
	

	if (const FGameInputDeviceConfiguration* Config = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo()))
	{
		// Reset the button state to not being pressed
		// Just use 0 as our button mask because we want them all to be set to 0
		uint32 CurrentButtonHeldMask = 0;

		EvaluateButtonStates(
			Params,
			CurrentButtonHeldMask,
			LastButtonHeldMask,
			RepeatTime,
			Config->ControllerButtonMappingData,
			MaxSupportedButtons);
			
		// Reset the axis state to be zero on all axis that we had in the previous state
		for (int32 i = 0; i < PreviousControllerAxisValues.Num(); ++i)
		{
			if (const FGameInputControllerAxisData* AxisData = Config->ControllerAxisMappingData.Find(i))
			{
				OnControllerAnalog(Params, AxisData->KeyName, 0.0f, PreviousControllerAxisValues[i], AxisData->DeadZone);
			}			
		}

		FMemory::Memset(PreviousControllerAxisValues, 0);
	}
}

GameInputKind FGameInputControllerDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindController | GameInputKindControllerAxis | GameInputKindControllerButton;
}

bool FGameInputControllerDeviceProcessor::ProcessControllerButtonState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* ControllerConfig, IGameInputReading* InputReading)
{
	// We can only process generic controllers that have their config set up
	// but allow you to run without a config to log out button indexes to make it 
	// easier to configure your device
	if (ControllerConfig && !ControllerConfig->bProcessControllerButtons)
	{
		return false;
	}

	const uint32 ButtonCount = InputReading->GetControllerButtonCount();

	TArray<bool> ButtonStates;
	ButtonStates.AddUninitialized(ButtonCount);

	const uint32 Res = InputReading->GetControllerButtonState(ButtonCount, ButtonStates.GetData());
	if (!Res)
	{
		return false;
	}
	
	uint32 CurrentButtonHeldMask = 0x00;

	for (uint32 i = 0; i < ButtonCount; ++i)
	{
		const FName* KeyName = ControllerConfig ? ControllerConfig->ControllerButtonMappingData.Find(1 << i) : nullptr;

		CurrentButtonHeldMask |= (ButtonStates[i] << i);

		if (ButtonStates[i])
		{
			UE_LOGF(LogGameInput, Verbose, "[ProcessControllerButtonState] Device ID: %d   Button count: %d    index: %d  State: %d   %ls", 
				Params.InputDeviceId.GetId(),
				ButtonCount, 
				i,
				ButtonStates[i], 
				KeyName ? *(KeyName->ToString()) : TEXT("NONE"));
		}
	}

	if (ControllerConfig)
	{
		EvaluateButtonStates(
			Params,
			CurrentButtonHeldMask,
			OUT LastButtonHeldMask,
			RepeatTime,
			ControllerConfig->ControllerButtonMappingData,
			ButtonCount
		);
	}	

	// Note: Ideally we would try and use the GameInput label system, 
	// but it is currently unfinished within the GameInput API itself so we can't.
	// We have provided this device specific config driven option instead.

	return true;
}

bool FGameInputControllerDeviceProcessor::ProcessControllerAxisState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* Config, IGameInputReading* InputReading)
{
	if (Config && !Config->bProcessControllerAxis)
	{
		return false;
	}

	const uint32 AxisCount = InputReading->GetControllerAxisCount();
	TArray<float> AxisValues;
	AxisValues.AddZeroed(AxisCount);
	if (!InputReading->GetControllerAxisState(AxisCount, AxisValues.GetData()))
	{
		return false;
	}

	// Make sure that there is previous controller data initialized if necessary
	while (PreviousControllerAxisValues.Num() < static_cast<int32>(AxisCount))
	{
		PreviousControllerAxisValues.AddZeroed();
	}

	for (uint32 i = 0; i < AxisCount; ++i)
	{
		float CurrentValue = AxisValues[i];
		const float PreviousValue = PreviousControllerAxisValues[i];

		if (Config)
		{
			if (const FGameInputControllerAxisData* AxisData = Config->ControllerAxisMappingData.Find(i))
			{
				if (AxisData->KeyName.IsValid() && AxisData->KeyName != NAME_None)
				{
					if (AxisData->bIsPackedPositveAndNegative)
					{
						// Maps the value to be -1.0 to +1.0
						CurrentValue = (CurrentValue * 2.f) - 1.f;
					}

					CurrentValue *= AxisData->Scalar;
					
					OnControllerAnalog(Params, AxisData->KeyName, CurrentValue, PreviousValue, AxisData->DeadZone);

					// Store this value for the next frame to compare to
					PreviousControllerAxisValues[i] = CurrentValue;
				}
				else
				{
					UE_LOGF(LogGameInput, VeryVerbose, "[ProcessControllerAxisState] (Device %ls) Invalid key name configured for controller axis '%d': %.3f", *UE::GameInput::LexToString(Params.Device), i, CurrentValue);
				}
			}
			else
			{
				// TODO: Here is where could send a "Generic USB Axis X" key here which could allow for us to support many more devices via a key rebind screen
				UE_LOGF(LogGameInput, VeryVerbose, "[ProcessControllerAxisState] (Device %ls) Controller axis '%d' has value: %.3f", *UE::GameInput::LexToString(Params.Device), i, CurrentValue);
			}
		}		
		// You are receiving analog values from an axis that you might not know about, log it here
		// in case you are trying to set something up
		else
		{
			UE_LOGF(LogGameInput, VeryVerbose, "[ProcessControllerAxisState] (Device %ls) Receiving input from an unconfigured controller axis '%d': %.3f", *UE::GameInput::LexToString(Params.Device), i, CurrentValue);
		}
	}

	return false;
}

bool FGameInputControllerDeviceProcessor::ProcessControllerSwitchState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* Config, IGameInputReading* InputReading)
{
	if (Config && !Config->bProcessControllerSwitchState)
	{
		return false;
	}

	const uint32 SwitchCount = InputReading->GetControllerSwitchCount();
	TArray<GameInputSwitchPosition> SwitchPositions;
	SwitchPositions.AddUninitialized(SwitchCount);

	// Make sure that we have some previous state initialized if we can
	if (static_cast<uint32>(PreviousSwitchPositions.Num()) < SwitchCount)
	{
		PreviousSwitchPositions.AddDefaulted(SwitchCount);
	}

	uint32 Res = InputReading->GetControllerSwitchState(SwitchCount, SwitchPositions.GetData());

	if (!Res)
	{
		return false;
	}

	// Evaluate all switch positions
	for (uint32 i = 0; i < SwitchCount; ++i)
	{
		// The PreviousSwitchPositions will be updated to the current switch position by the evaluate function
		EvaluateSwitchState(
			Params,
			SwitchPositions[i],
			OUT PreviousSwitchPositions[i],
			SwitchRepeatTimes);
	}

	return true;
}

#endif	// #if GAME_INPUT_SUPPORT

