// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_FlightStick.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputKeyTypes.h"

#if GAME_INPUT_SUPPORT

namespace UE::GameInput
{
	/** A map of uint32 GameInput button bitmask flags to the associated Unreal Engine FKey name. */
	static const TMap<uint32, FGamepadKeyNames::Type>& GetFlightStickButtonMap()
	{
		static const TMap<uint32, FGamepadKeyNames::Type> GamepadButtonMap
		{
			// Generic gamepad buttons
			{ static_cast<uint32>(GameInputFlightStickButtons::GameInputFlightStickNone), FGamepadKeyNames::Invalid},
			{ static_cast<uint32>(GameInputFlightStickButtons::GameInputFlightStickMenu), FGamepadKeyNames::Invalid},
			{ static_cast<uint32>(GameInputFlightStickButtons::GameInputFlightStickView), FGamepadKeyNames::Invalid},
			{ static_cast<uint32>(GameInputFlightStickButtons::GameInputFlightStickFirePrimary), FGamepadKeyNames::Invalid},
			{ static_cast<uint32>(GameInputFlightStickButtons::GameInputFlightStickFireSecondary), FGamepadKeyNames::Invalid},
		};
		return GamepadButtonMap;
	}
	
	static bool HasDifferentFlightStickAnalogs(const GameInputFlightStickState& CurrentState, const GameInputFlightStickState& PreviousState)
	{
		// If anything differs from the previous reading, then it has different inputs
		return 
			CurrentState.pitch != PreviousState.pitch ||
			CurrentState.roll != PreviousState.roll ||
			CurrentState.throttle != PreviousState.throttle ||
			CurrentState.yaw != PreviousState.yaw;
	}

	inline bool IsEmptyFlightStickReading(const GameInputFlightStickState& State, const UGameInputPlatformSettings& PlatformSettings)
	{
		return
			State.buttons == 0 &&
			State.hatSwitch == 0 &&	
			FMath::Abs(State.pitch) <= PlatformSettings.FlightStickPitchDeadzone &&
			FMath::Abs(State.roll) <= PlatformSettings.FlightStickRollDeadzone &&
			FMath::Abs(State.throttle) <= PlatformSettings.FlightStickThrottleDeadzone &&
			FMath::Abs(State.yaw) <= PlatformSettings.FlightStickYawDeadzone; 
	};
}

FGameInputFlightStickProcessor::FGameInputFlightStickProcessor()
	: IGameInputDeviceProcessor()
{
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
	
	SwitchRepeatTimes.AddDefaulted(static_cast<uint32>(GameInputSwitchUpLeft) + 1);
}

bool FGameInputFlightStickProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;
	
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid() || !Params.Reading)
	{
		return bRes;
	}

	GameInputFlightStickState FlightStickState = {};
	if (!Params.Reading->GetFlightStickState(&FlightStickState))
	{
		return bRes;
	}

	// We only want to process the buttons here, as it might get called multiple times per frame.
	bRes |= ProcessFlightStickButtons(Params, FlightStickState);
	
	++NumReadingsProcessedThisFrame;
	
	return bRes;
}

bool FGameInputFlightStickProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;
	
	// Check if we have already processed buttons this frame. If we haven't we want to do it
	const bool bHasProcessedAnyButtonsThisFrame = NumReadingsProcessedThisFrame > 0;	
	NumReadingsProcessedThisFrame = 0;
	
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid() || !Params.PreviousReading)
	{
		return bRes;
	}

	GameInputFlightStickState FlightStickState = {};
	if (!Params.PreviousReading->GetFlightStickState(&FlightStickState))
	{
		return bRes;
	}
	
	if (!bHasProcessedAnyButtonsThisFrame)
	{
		bRes |= ProcessFlightStickButtons(Params, FlightStickState);
	}
	
	bRes |= ProcessFlightStickAnalog(Params, FlightStickState);
	
	return bRes;
}

void FGameInputFlightStickProcessor::ClearState(const FGameInputEventParams& Params)
{
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return;
	}
	
	// Process input as if nothing is down (zero values for everything)
	GameInputFlightStickState ZeroState = {};
	ProcessFlightStickAnalog(Params, ZeroState);
	ProcessFlightStickButtons(Params, ZeroState);
	
	// Zero out the repeat and previous states info to zero
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
}

GameInputKind FGameInputFlightStickProcessor::GetSupportedReadingKind() const
{
	return GameInputKindFlightStick;
}

bool FGameInputFlightStickProcessor::ProcessFlightStickButtons(const FGameInputEventParams& Params, GameInputFlightStickState& State)
{
	const UGameInputPlatformSettings* PlatformSettings = UGameInputPlatformSettings::Get();

	const bool bIsEmptyReading = UE::GameInput::IsEmptyFlightStickReading(State, *PlatformSettings);
	const bool bWasEmptyReading = UE::GameInput::IsEmptyFlightStickReading(PreviousState, *PlatformSettings);
	const bool bHasDifferentAnalogInput = UE::GameInput::HasDifferentFlightStickAnalogs(State, PreviousState);
	
	if (bIsEmptyReading && bWasEmptyReading && !bHasDifferentAnalogInput && PreviousState.buttons == 0)
	{
		return false;
	}
	
	const uint32 CurrentButtonHeldMask = static_cast<uint32>(State.buttons);
	uint32 LastButtonHeldMask = static_cast<uint32>(PreviousState.buttons);

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		OUT LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetFlightStickButtonMap(),
		MaxSupportedButtons);

	// Update the previous state here
	PreviousState.buttons = State.buttons;

	// Update the hat switch
	EvaluateSwitchState(Params, State.hatSwitch, PreviousState.hatSwitch, SwitchRepeatTimes);
	
	return true;
}

bool FGameInputFlightStickProcessor::ProcessFlightStickAnalog(const FGameInputEventParams& Params, GameInputFlightStickState& State)
{
	const UGameInputPlatformSettings* Settings = UGameInputPlatformSettings::Get();
	check(Settings);
	
	// If the analog values haven't changed, don't bother sending any events for them
	const bool bIsEmptyReading = UE::GameInput::IsEmptyFlightStickReading(State, *Settings);
	const bool bWasEmptyReading = UE::GameInput::IsEmptyFlightStickReading(PreviousState, *Settings);
	const bool bHasDifferentAnalogInput = UE::GameInput::HasDifferentFlightStickAnalogs(State, PreviousState);
	if (bIsEmptyReading && bWasEmptyReading && !bHasDifferentAnalogInput)
	{
		return false;
	}
	
	OnControllerAnalog(Params, FGameInputKeys::FlightStick_Pitch.GetFName(), State.pitch, PreviousState.pitch, Settings->FlightStickPitchDeadzone);
	OnControllerAnalog(Params, FGameInputKeys::FlightStick_Roll.GetFName(), State.roll, PreviousState.roll, Settings->FlightStickRollDeadzone);
	OnControllerAnalog(Params, FGameInputKeys::FlightStick_Throttle.GetFName(), State.throttle, PreviousState.throttle, Settings->FlightStickThrottleDeadzone);
	OnControllerAnalog(Params, FGameInputKeys::FlightStick_Yaw.GetFName(), State.yaw, PreviousState.yaw, Settings->FlightStickYawDeadzone);

	PreviousState.pitch = State.pitch;
	PreviousState.roll = State.roll;
	PreviousState.throttle = State.throttle;
	PreviousState.yaw = State.yaw;
	
	return true;
}

#endif