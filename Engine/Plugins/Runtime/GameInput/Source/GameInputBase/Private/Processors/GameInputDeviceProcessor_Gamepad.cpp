// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_Gamepad.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputUtils.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

#if GAME_INPUT_SUPPORT

namespace UE::GameInput
{
	inline bool IsEmptyGamepadReading(const GameInputGamepadState& GamepadState)
	{
		return GamepadState.buttons == 0 &&
			GamepadState.leftTrigger <= GamepadTriggerDeadzone && GamepadState.rightTrigger <= GamepadTriggerDeadzone &&
			FMath::Abs(GamepadState.leftThumbstickX) <= GamepadLeftStickDeadzone && FMath::Abs(GamepadState.leftThumbstickY) <= GamepadLeftStickDeadzone &&
			FMath::Abs(GamepadState.rightThumbstickX) <= GamepadRightStickDeadzone && FMath::Abs(GamepadState.rightThumbstickY) <= GamepadRightStickDeadzone;
	};

	inline bool HasDifferentAnalogInput(const GameInputGamepadState& CurrentGamepadState, const GameInputGamepadState& PreviousGamepadState)
	{
		return CurrentGamepadState.leftTrigger != PreviousGamepadState.leftTrigger || CurrentGamepadState.rightTrigger != PreviousGamepadState.rightTrigger ||
			CurrentGamepadState.leftThumbstickX != PreviousGamepadState.leftThumbstickX || CurrentGamepadState.leftThumbstickY != PreviousGamepadState.leftThumbstickY ||
			CurrentGamepadState.rightThumbstickX != PreviousGamepadState.rightThumbstickX || CurrentGamepadState.rightThumbstickY != PreviousGamepadState.rightThumbstickY;
	};

	/**
	 * Returns a map of GameInputGamepadButtons to their associated Unreal Engine FKey names.
	 */
	static const TMap<uint32, FGamepadKeyNames::Type>& GetGamepadButtonMap()
	{
		static const TMap<uint32, FGamepadKeyNames::Type> GamepadButtonMap
		{
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadA), FGamepadKeyNames::FaceButtonBottom },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadB), FGamepadKeyNames::FaceButtonRight },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadX), FGamepadKeyNames::FaceButtonLeft },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadY), FGamepadKeyNames::FaceButtonTop },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadLeftShoulder), FGamepadKeyNames::LeftShoulder },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadRightShoulder), FGamepadKeyNames::RightShoulder },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadMenu), FGamepadKeyNames::SpecialRight },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadView), FGamepadKeyNames::SpecialLeft },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadDPadUp), FGamepadKeyNames::DPadUp },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadDPadDown), FGamepadKeyNames::DPadDown },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadDPadLeft), FGamepadKeyNames::DPadLeft },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadDPadRight), FGamepadKeyNames::DPadRight },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadLeftThumbstick), FGamepadKeyNames::LeftThumb },
			{ static_cast<uint32>(GameInputGamepadButtons::GameInputGamepadRightThumbstick), FGamepadKeyNames::RightThumb },

			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_LeftTrigger), FGamepadKeyNames::LeftTriggerThreshold },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_RightTrigger), FGamepadKeyNames::RightTriggerThreshold },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_LeftStickUp), FGamepadKeyNames::LeftStickUp },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_LeftStickDown), FGamepadKeyNames::LeftStickDown },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_LeftStickLeft), FGamepadKeyNames::LeftStickLeft },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_LeftStickRight), FGamepadKeyNames::LeftStickRight },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_RightStickUp), FGamepadKeyNames::RightStickUp },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_RightStickDown), FGamepadKeyNames::RightStickDown },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_RightStickLeft), FGamepadKeyNames::RightStickLeft },
			{ static_cast<uint32>(EGameInputAuxButton::GamepadButtonAux_RightStickRight), FGamepadKeyNames::RightStickRight }
		};
		return GamepadButtonMap;
	}
}

bool FGameInputGamepadDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	// check if the reading had gamepad info
	GameInputGamepadState GamepadState;
	if (!Params.Reading->GetGamepadState(&GamepadState))
	{
		return false;
	}

	bool bRes = false;	

	// We want to process gamepad BUTTON states for every game input reading.
	bRes |= ProcessGamepadButtonState(Params, GamepadState);
	++NumReadingsProcessedThisFrame;

	return bRes;
}

bool FGameInputGamepadDeviceProcessor::PostProcessInput(const FGameInputEventParams& Params)
{	
	// On the last input reading for the frame, the "Current Reading" should always be null. we only care for the LastReading here
	ensure (Params.Reading == nullptr);

	const bool bProcessedAnyGamepadButtonsThisFrame = (NumReadingsProcessedThisFrame > 0);

	// This is the last reading of this frame, reset the counter to 0
	NumReadingsProcessedThisFrame = 0;

	if (!Params.PreviousReading)
	{
		return false;
	}

	// Get the gamepad state of the *Last Reading* of the frame here
	GameInputGamepadState GamepadState;
	if (!Params.PreviousReading->GetGamepadState(&GamepadState))
	{
		return false;
	}

	bool bRes = false;
	
	// We only want to process gamepad ANALOG inputs on the last reading because we only care about
	// the most recent analog stick input value. If there are multiple readings and we processed analog
	// for every one, then the values would accumulate and stack, giving us incorrect data in the message handler.
	bRes |= ProcessGamepadAnalogState(Params, GamepadState);
	
	// If there were no gamepad events this frame, send button events using the previous frame's reading for button repeats.
	// This is necessary because GetNextReading won't return any reading if the state is unchanged
	if (!bProcessedAnyGamepadButtonsThisFrame)
	{
		bRes |= ProcessGamepadButtonState(Params, GamepadState);
	}

	return bRes;
}

bool FGameInputGamepadDeviceProcessor::ProcessGamepadAnalogState(const FGameInputEventParams& Params, GameInputGamepadState& GamepadState)
{
	if (!Params.PlatformUserId.IsValid())
	{
		return false;
	}

	// ignore this input if the reading has remained empty from last time
	const bool bIsEmptyReading = UE::GameInput::IsEmptyGamepadReading(GamepadState);
	const bool bWasEmptyReading = UE::GameInput::IsEmptyGamepadReading(PreviousState);
	const bool bHasDifferentAnalogInput = UE::GameInput::HasDifferentAnalogInput(GamepadState, PreviousState);
	if (bIsEmptyReading && bWasEmptyReading && !bHasDifferentAnalogInput)
	{
		return false;
	}

	OnControllerAnalog(Params, FGamepadKeyNames::LeftAnalogX, GamepadState.leftThumbstickX, PreviousState.leftThumbstickX, UE::GameInput::GamepadLeftStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::LeftAnalogY, GamepadState.leftThumbstickY, PreviousState.leftThumbstickY, UE::GameInput::GamepadLeftStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightAnalogX, GamepadState.rightThumbstickX, PreviousState.rightThumbstickX, UE::GameInput::GamepadRightStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightAnalogY, GamepadState.rightThumbstickY, PreviousState.rightThumbstickY, UE::GameInput::GamepadRightStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::LeftTriggerAnalog, GamepadState.leftTrigger, PreviousState.leftTrigger, UE::GameInput::GamepadTriggerDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightTriggerAnalog, GamepadState.rightTrigger, PreviousState.rightTrigger, UE::GameInput::GamepadTriggerDeadzone);
	
	// map analog triggers to digital input.
	uint32 AnalogButtonMask = 0u;
	if (GamepadState.leftTrigger > UE::GameInput::GamepadTriggerDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_LeftTrigger;
	}

	if (GamepadState.rightTrigger > UE::GameInput::GamepadTriggerDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_RightTrigger;
	}

	// map left/right stick digital inputs to (top 8 bits)
	if (GamepadState.leftThumbstickY > UE::GameInput::GamepadLeftStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_LeftStickUp;
	}
	else if (GamepadState.leftThumbstickY < -UE::GameInput::GamepadLeftStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_LeftStickDown;
	}
	if (GamepadState.leftThumbstickX > UE::GameInput::GamepadLeftStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_LeftStickRight;
	}
	else if (GamepadState.leftThumbstickX < -UE::GameInput::GamepadLeftStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_LeftStickLeft;
	}

	if (GamepadState.rightThumbstickY > UE::GameInput::GamepadRightStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_RightStickUp;
	}
	else if (GamepadState.rightThumbstickY < -UE::GameInput::GamepadRightStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_RightStickDown;
	}
	if (GamepadState.rightThumbstickX > UE::GameInput::GamepadRightStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_RightStickRight;
	}
	else if (GamepadState.rightThumbstickX < -UE::GameInput::GamepadRightStickDeadzone)
	{
		AnalogButtonMask |= UE::GameInput::EGameInputAuxButton::GamepadButtonAux_RightStickLeft;
	}

	// Evaluate the analog button mask
	EvaluateButtonStates(
		Params,
		AnalogButtonMask,
		OUT LastAnalogButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetGamepadButtonMap(),
		MaxSupportedButtons);
	
	// update saved analog state
	PreviousState.leftTrigger = GamepadState.leftTrigger;
	PreviousState.rightTrigger = GamepadState.rightTrigger;
	PreviousState.leftThumbstickX = GamepadState.leftThumbstickX;
	PreviousState.leftThumbstickY = GamepadState.leftThumbstickY;
	PreviousState.rightThumbstickX = GamepadState.rightThumbstickX;
	PreviousState.rightThumbstickY = GamepadState.rightThumbstickY;
		
	return true;
}

bool FGameInputGamepadDeviceProcessor::ProcessGamepadButtonState(const FGameInputEventParams& Params, GameInputGamepadState& GamepadState)
{
	if (!Params.PlatformUserId.IsValid())
	{
		return false;
	}

	// ignore this input if the reading has remained empty from last time, or we still have outstanding held buttons 
	// (these held buttons are likely to occur with mapped analog triggers when there have been multiple input events this frame)
	const bool bIsEmptyReading = UE::GameInput::IsEmptyGamepadReading(GamepadState);
	const bool bWasEmptyReading = UE::GameInput::IsEmptyGamepadReading(PreviousState);
	const bool bHasDifferentAnalogInput = UE::GameInput::HasDifferentAnalogInput(GamepadState, PreviousState);
	if (bIsEmptyReading && bWasEmptyReading && !bHasDifferentAnalogInput && LastButtonHeldMask == 0)
	{
		return false;
	}

	// map buttons (low 15 bits)
	uint32 CurrentButtonHeldMask = (static_cast<uint32>(GamepadState.buttons) & UE::GameInput::GamingInputButtonMask);

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		OUT LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetGamepadButtonMap(),
		MaxSupportedButtons);

	// Keep track of the current BUTTON state. We don't want to update the entire PreviousState struct here
	// because buttons may be evaluated more then analog inputs per-frame
	PreviousState.buttons = GamepadState.buttons;

	return true;
}

void FGameInputGamepadDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// We need a valid input device id when sending messages, otherwise slate will hit a check
	// and attempt to create some new slate user with an invalid index.
	if (!Params.PlatformUserId.IsValid() || !Params.InputDeviceId.IsValid())
	{
		return;
	}

	// Reset Axis values
	OnControllerAnalog(Params, FGamepadKeyNames::LeftAnalogX, 0.0f, PreviousState.leftThumbstickX, UE::GameInput::GamepadLeftStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::LeftAnalogY, 0.0f, PreviousState.leftThumbstickY, UE::GameInput::GamepadLeftStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightAnalogX, 0.0f, PreviousState.rightThumbstickX, UE::GameInput::GamepadRightStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightAnalogY, 0.0f, PreviousState.rightThumbstickY, UE::GameInput::GamepadRightStickDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::LeftTriggerAnalog, 0.0f, PreviousState.leftTrigger, UE::GameInput::GamepadTriggerDeadzone);
	OnControllerAnalog(Params, FGamepadKeyNames::RightTriggerAnalog, 0.0f, PreviousState.rightTrigger, UE::GameInput::GamepadTriggerDeadzone);

	// Reset button values

	// Just use 0 as our button mask because we want them all to be set to 0
	uint32 CurrentButtonHeldMask = 0;

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetGamepadButtonMap(),
		MaxSupportedButtons);

	// Clear the analog button mask
	LastAnalogButtonHeldMask = 0u;

	for (uint32 i = 0; i < MaxSupportedButtons; i++)
	{
		RepeatTime[i] = 0.0;
	}

	// clear previous gamepad state
	FMemory::Memset(PreviousState, 0);
}

GameInputKind FGameInputGamepadDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindGamepad;
}

#endif	// #if GAME_INPUT_SUPPORT