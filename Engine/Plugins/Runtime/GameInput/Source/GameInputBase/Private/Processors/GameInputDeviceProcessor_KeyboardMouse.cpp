// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_KeyboardMouse.h"

#include "Framework/Application/SlateApplication.h"
#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "GenericPlatform/CursorUtils.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformTime.h"

#if GAME_INPUT_SUPPORT

static float GKeyboardRepeatInitialDelay = 0.25f;
static FAutoConsoleVariableRef CVarKeyboardRepeatInitialDelay(
	TEXT("Input.KeyboardRepeatInitialDelay"),
	GKeyboardRepeatInitialDelay,
	TEXT("Time in seconds before a key repeat starts"));

static float GKeyboardRepeatDelay = 0.05f;
static FAutoConsoleVariableRef CVarKeyboardRepeatDelay(
	TEXT("Input.KeyboardRepeatDelay"),
	GKeyboardRepeatDelay,
	TEXT("Time in seconds between each subsequent key repeat"));

bool FGameInputKeyboardDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{	
	TSet<uint8> CurrentPressedKeys;

	int32 KeyCount = Params.Reading->GetKeyCount();
	if (KeyCount > 0)
	{
		// read the key state
		TArray<GameInputKeyState> KeyStates;
		KeyStates.AddUninitialized(KeyCount);
		int32 ReadCount = Params.Reading->GetKeyState(KeyCount, KeyStates.GetData());

		// build a set of the pressed keycodes
		for (GameInputKeyState KeyState : KeyStates)
		{
			CurrentPressedKeys.Add(KeyState.virtualKey);
		}		
	}

	UpdateUnifiedKeyboardState(Params, CurrentPressedKeys);
	return true;
}

void FGameInputKeyboardDeviceProcessor::UpdateUnifiedKeyboardState(const FGameInputEventParams& Params, TSet<uint8>& CurrentPressedKeys)
{
	// process unified pressed keys
	double CurrentTime = FPlatformTime::Seconds();
	for (uint8 KeyCode : CurrentPressedKeys)
	{
		bool bIsRepeat = LastPressedKeys.Contains(KeyCode);
		if (!bIsRepeat)
		{
			KeyRepeatTime.Add(KeyCode, CurrentTime + GKeyboardRepeatInitialDelay);
			Params.MessageHandler->OnKeyDown(KeyCode, 0, false);
			UE_LOGF(LogGameInput, Verbose, "Key Press 0x%X", KeyCode);

			if (KeyCode == VK_CAPITAL)
			{
				SetSimulatedCapsLock(!bSimulatedCapsLock);
				UE_LOGF(LogGameInput, Verbose, "Simulated caps lock is %ls", bSimulatedCapsLock ? TEXT("ON") : TEXT("OFF"));
			}
		}
		else if (CurrentTime > KeyRepeatTime[KeyCode])
		{
			KeyRepeatTime.Add(KeyCode, CurrentTime + GKeyboardRepeatDelay);
			Params.MessageHandler->OnKeyDown(KeyCode, 0, true);
			UE_LOGF(LogGameInput, Verbose, "Key Press 0x%X (repeat)", KeyCode);
		}
	}

	// process any released keys
	for (uint8 KeyCode : LastPressedKeys)
	{
		if (!CurrentPressedKeys.Contains(KeyCode))
		{
			KeyRepeatTime.Remove(KeyCode);
			Params.MessageHandler->OnKeyUp(KeyCode, 0, false);
			UE_LOGF(LogGameInput, Verbose, "Key Release 0x%X", KeyCode);
		}
	}

	// update saved state
	LastPressedKeys = CurrentPressedKeys;
}

bool FGameInputKeyboardDeviceProcessor::IsKeyPressed(uint8 VirtualKeyCode) const
{
	return LastPressedKeys.Contains(VirtualKeyCode);
}

void FGameInputKeyboardDeviceProcessor::SetSimulatedCapsLock(bool bVal)
{
	bSimulatedCapsLock = bVal;
}

void FGameInputKeyboardDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	TSet<uint8> NoPressedKeys;
	UpdateUnifiedKeyboardState(Params, NoPressedKeys);
}

GameInputKind FGameInputKeyboardDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindKeyboard;
}

static int GAllowVirtualMouseInput = 1;
static FAutoConsoleVariableRef CVarAllowVirtualMouseInput(
	TEXT("GameInput.AllowVirtualMouseInput"),
	GAllowVirtualMouseInput,
	TEXT("Whether to accept input from virtual mice, such as those from a remote viewer. Note that this doesn't change whether the mouse is 'connected'"));

static float GMouseSensitivity = 1.0f;
static FAutoConsoleVariableRef CVarMouseSensitivity(
	TEXT("Input.MouseSensitivity"),
	GMouseSensitivity,
	TEXT("The sensitivity multiplier of the mouse\n")
	TEXT(" 1 (default)"),
	ECVF_Default);

static float GMouseDoubleClickArea = 10.0f;
static FAutoConsoleVariableRef CVarMouseDoubleClickArea(
	TEXT("Input.DoubleClickArea"),
	GMouseDoubleClickArea,
	TEXT("How far the mouse can move between double clicks to still count as a double click"));

static float GMouseDoubleClickDelay = 0.5f;
static FAutoConsoleVariableRef CVarMouseDoubleClickDelay(
	TEXT("Input.DoubleClickDelay"),
	GMouseDoubleClickDelay,
	TEXT("Time in seconds between mouse down events to trigger a double click event. Set to 0 to disable double clicking"));

namespace UE::GameInput
{
	struct FMouseButtonMapping
	{
		uint32 GameInputButton;
		EMouseButtons::Type MouseButton;
	};
	static const FMouseButtonMapping MouseButtonMappings[] =
	{
		{ static_cast<uint32>(GameInputMouseButtons::GameInputMouseLeftButton), EMouseButtons::Left    },
		{ static_cast<uint32>(GameInputMouseButtons::GameInputMouseRightButton), EMouseButtons::Right   },
		{ static_cast<uint32>(GameInputMouseButtons::GameInputMouseMiddleButton), EMouseButtons::Middle  },
		{ static_cast<uint32>(GameInputMouseButtons::GameInputMouseButton4), EMouseButtons::Thumb01 },
		{ static_cast<uint32>(GameInputMouseButtons::GameInputMouseButton5), EMouseButtons::Thumb02 },
	};
}

FGameInputMouseDeviceProcessor::FGameInputMouseDeviceProcessor(const TSharedPtr<class ICursor>& InCursor)
	: Cursor(InCursor)
	, LastMouseOffset(ForceInitToZero)
{
	FMemory::Memset(PreviousMouseState, 0);

	for (uint32 i = 0; i < MaxSupportedButtons; i++)
	{
		RepeatTime[i] = 0.0;
	}
}

bool FGameInputMouseDeviceProcessor::CanProcessVirtualMouse() const
{
	// ignore the input if requested
	if (GAllowVirtualMouseInput == 0)
	{
		return false;
	}
	return true;
}

bool FGameInputMouseDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	// read mouse state
	GameInputMouseState MouseState;
	if (!Params.Reading->GetMouseState(&MouseState))
	{
		return false;
	}

	int32 LEGACY_VirtualMaxX = MAX_int32;
	int32 LEGACY_VirtualMaxY = MAX_int32;

	const GameInputDeviceInfo* DeviceInfo = Params.GetDeviceInfo();
	const bool bIsVirtualMouse = DeviceInfo ? DeviceInfo->deviceFamily == GameInputFamilyVirtual : false;

	const bool bHighPrecisionMouseMode = FSlateApplication::Get().IsUsingHighPrecisionMouseMovment();

	if (bIsVirtualMouse)
	{
		if (!CanProcessVirtualMouse())
		{
			return false;
		}
	}

	// update mouse position
	float MouseDX, MouseDY = 0.0f;
	
	int32 RawMouseDX = static_cast<int32>(MouseState.positionX) - static_cast<int32>(PreviousMouseState.positionX);
	int32 RawMouseDY = static_cast<int32>(MouseState.positionY) - static_cast<int32>(PreviousMouseState.positionY);	
	
	if (bHighPrecisionMouseMode)
	{
		MouseDX = static_cast<float>(RawMouseDX);
		MouseDY = static_cast<float>(RawMouseDY);
	}
	else
	{
		MouseDX = UE::Cursor::CalculateDeltaWithAcceleration(RawMouseDX, GMouseSensitivity);
		MouseDY = UE::Cursor::CalculateDeltaWithAcceleration(RawMouseDY, GMouseSensitivity);
	}
	if (MouseDX != 0 || MouseDY != 0)
	{
		if (Cursor.IsValid())
		{
			FVector2D CursorPos = Cursor->GetPosition();
			CursorPos.X += MouseDX;
			CursorPos.Y += MouseDY;
			Cursor->SetPosition((int32)CursorPos.X, (int32)CursorPos.Y);
		}

		// LastMouseOffset is used for double-click detection, so we should use the same coordinates as the cursor i.e. the processed ones.
		LastMouseOffset.X += MouseDX;
		LastMouseOffset.Y += MouseDY;

		UE_LOGF(LogGameInput, VeryVerbose, "Device %ls (InputDeviceId = %d) - Mouse RawDX %d, RawDY %d, DX %0.2f, DY %.2f  %ls", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), RawMouseDX, RawMouseDY, MouseDX, MouseDY, bHighPrecisionMouseMode ? TEXT("(high precision)") : TEXT(""));

		// OnRawMouseMove may be used to process player character controls so, as the name suggests, these should be raw data without any smoothing.
		Params.MessageHandler->OnRawMouseMove(RawMouseDX, RawMouseDY);
	}

	// update mouse wheel
	const float MouseWheelSpinFactor = 1 / 120.0f;
	float MouseWheelDX = (float)(MouseState.wheelX - PreviousMouseState.wheelX);
	float MouseWheelDY = (float)(MouseState.wheelY - PreviousMouseState.wheelY);
	if (MouseWheelDY != 0)
	{
		UE_LOGF(LogGameInput, Verbose, "Device %ls (InputDeviceId = %d) - Mouse Wheel DY %.2f", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), MouseWheelDY);
		Params.MessageHandler->OnMouseWheel(MouseWheelDY * MouseWheelSpinFactor);
	}

	// handle button change events
	uint32 CurrentButtonHeldMask = (uint32)MouseState.buttons;
	uint32 ActionMask = (LastButtonHeldMask ^ CurrentButtonHeldMask);
	uint32 RepeatMask = (LastButtonHeldMask & CurrentButtonHeldMask);
	double CurrentTime = FPlatformTime::Seconds();

	for (int ButtonIndex = 0; ButtonIndex < UE_ARRAY_COUNT(UE::GameInput::MouseButtonMappings); ButtonIndex++)
	{
		const uint32 GameInputButton = UE::GameInput::MouseButtonMappings[ButtonIndex].GameInputButton;
		if ((ActionMask & GameInputButton) == 0)
		{
			continue;
		}

		const EMouseButtons::Type MouseButton = UE::GameInput::MouseButtonMappings[ButtonIndex].MouseButton;
		if ((CurrentButtonHeldMask & GameInputButton) != 0)
		{
			const bool bHasDoubleClick = (GMouseDoubleClickDelay > 0) && (CurrentTime <= RepeatTime[ButtonIndex]) && (LastMouseOffset.SizeSquared() <= FMath::Square(GMouseDoubleClickArea));
			if (bHasDoubleClick)
			{
				UE_LOGF(LogGameInput, Verbose, "Device %ls (DeviceIndex = %d) - %ls Double Click", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), UE::GameInput::GetMouseButtonName(MouseButton));
				Params.MessageHandler->OnMouseDoubleClick(nullptr, MouseButton);
			}
			else
			{
				UE_LOGF(LogGameInput, Verbose, "Device %ls (DeviceIndex = %d) - %ls Pressed", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), UE::GameInput::GetMouseButtonName(MouseButton));
				Params.MessageHandler->OnMouseDown(nullptr, MouseButton);
			}

			RepeatTime[ButtonIndex] = CurrentTime + GMouseDoubleClickDelay;
			LastMouseOffset = FVector2D::ZeroVector;
		}
		else
		{
			UE_LOGF(LogGameInput, Verbose, "Device %ls (DeviceIndex = %d) - %ls Released", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), UE::GameInput::GetMouseButtonName(MouseButton));
			Params.MessageHandler->OnMouseUp(MouseButton);
		}
	}

	PreviousMouseState = MouseState;
	LastButtonHeldMask = CurrentButtonHeldMask;

	return true;
}

void FGameInputMouseDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// clear buttons
	for (int32 ButtonIndex = 0; ButtonIndex < UE_ARRAY_COUNT(UE::GameInput::MouseButtonMappings); ButtonIndex++)
	{
		uint32 GameInputButton = UE::GameInput::MouseButtonMappings[ButtonIndex].GameInputButton;
		if ((GameInputButton & LastButtonHeldMask) != 0)
		{
			EMouseButtons::Type MouseButton = UE::GameInput::MouseButtonMappings[ButtonIndex].MouseButton;
			UE_LOGF(LogGameInput, Verbose, "Device %ls (InputDeviceId = %d) - %ls Released (via ClearState)", *UE::GameInput::LexToString(Params.Device), Params.InputDeviceId.GetId(), UE::GameInput::GetMouseButtonName(MouseButton));
			Params.MessageHandler->OnMouseUp(MouseButton);
		}
	}

	// Clear repeat times
	for (uint32 i = 0; i < MaxSupportedButtons; i++)
	{
		RepeatTime[i] = 0.0;
	}

	// clear previous mouse state
	LastMouseOffset = FVector2D::ZeroVector;
	FMemory::Memset(PreviousMouseState, 0);
}

GameInputKind FGameInputMouseDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindMouse;
}

#endif	// #if GAME_INPUT_SUPPORT