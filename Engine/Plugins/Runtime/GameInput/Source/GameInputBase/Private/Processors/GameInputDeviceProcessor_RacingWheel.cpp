// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_RacingWheel.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputKeyTypes.h"
#include "HAL/ConsoleManager.h"

#if GAME_INPUT_SUPPORT

namespace UE::GameInput
{
	/**
	* Returns a map of GameInputRacingWheelButtons to their associated Unreal Engine FKey names.
	*/
	static const TMap<uint32, FGamepadKeyNames::Type>& GetRacingWheelButtonMap()
	{
		static const TMap<uint32, FGamepadKeyNames::Type> RacingWheelButtonMap
		{
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelNone), FGameInputKeys::RacingWheel_None.GetFName() },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelMenu), FGameInputKeys::RacingWheel_Menu.GetFName() },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelView), FGameInputKeys::RacingWheel_View.GetFName() },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelPreviousGear), FGameInputKeys::RacingWheel_PreviousGear.GetFName() },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelNextGear), FGameInputKeys::RacingWheel_NextGear.GetFName() },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelDpadUp), FGamepadKeyNames::DPadUp },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelDpadDown), FGamepadKeyNames::DPadDown },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelDpadLeft), FGamepadKeyNames::DPadLeft },
				{ static_cast<uint32>(GameInputRacingWheelButtons::GameInputRacingWheelDpadRight), FGamepadKeyNames::DPadRight },
			};

		return RacingWheelButtonMap;
	}
}; // namespace UE::GameInput

FGameInputRacingWheelProcessor::FGameInputRacingWheelProcessor()
	: IGameInputDeviceProcessor()
{
	NumReadingsProcessedThisFrame = 0;
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
}

bool FGameInputRacingWheelProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid() || !Params.Reading)
	{
		return bRes;
	}

	GameInputRacingWheelState WheelState;
	if (!Params.Reading->GetRacingWheelState(&WheelState))
	{
		return bRes;
	}

	// We only want to process the buttons here, as it might get called multiple times per frame.
	bRes |= ProcessWheelButtonState(Params, WheelState);

	++NumReadingsProcessedThisFrame;

	return bRes;
}

bool FGameInputRacingWheelProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	// Check if we have already processed buttons this frame. If we haven't we want to do it
	const bool bHasProcessedAnyButtonsThisFrame = NumReadingsProcessedThisFrame > 0;	
	NumReadingsProcessedThisFrame = 0;

	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid() || !Params.PreviousReading)
	{
		return bRes;
	}

	// Use the "PreviousRading" because we only want to process the analog inputs once, and this will
	// point to the most up to date reading.
	GameInputRacingWheelState WheelState;
	if (!Params.PreviousReading->GetRacingWheelState(&WheelState))
	{
		return bRes;
	}

	if (!bHasProcessedAnyButtonsThisFrame)
	{
		bRes |= ProcessWheelButtonState(Params, WheelState);
	}

	bRes |= ProcessWheelAnalogState(Params, WheelState);

	return bRes;
}

void FGameInputRacingWheelProcessor::ClearState(const FGameInputEventParams& Params)
{
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return;
	}

	// We can simply process a wheel state where everything is zero
	GameInputRacingWheelState ZeroValueState = {};
	FMemory::Memset(ZeroValueState, 0);

	ProcessWheelButtonState(Params, ZeroValueState);
	ProcessWheelAnalogState(Params, ZeroValueState);

	// Zero out any state trackers
	NumReadingsProcessedThisFrame = 0;
	FMemory::Memset(PreviousState, 0);
	FMemory::Memset(RepeatTime, 0);
}

namespace UE::GameInput
{
	static bool HasDifferentWheelAnalogInput(const GameInputRacingWheelState& CurrentState, const GameInputRacingWheelState& PreviousState)
	{
		// If anything differs from the previous reading, then it has different inputs
		return 
			CurrentState.wheel != PreviousState.wheel ||
			CurrentState.throttle != PreviousState.throttle ||
			CurrentState.brake != PreviousState.brake ||
			CurrentState.clutch != PreviousState.clutch ||
			CurrentState.handbrake != PreviousState.handbrake;
	}
};

const float FGameInputRacingWheelProcessor::GetRacingWheelDeadzone()
{
	// These settings should always be available
	if (const UGameInputPlatformSettings* Settings = UGameInputPlatformSettings::Get())
	{
		return Settings->RacingWheelDeadzone;
	}

	return UGameInputPlatformSettings::DefaultRacingWheelDeadzone;
}

bool FGameInputRacingWheelProcessor::ProcessWheelAnalogState(const FGameInputEventParams& Params, GameInputRacingWheelState& CurrentWheelState)
{
	// ignore this input if the reading has remained empty from last time
	const float Deadzone = GetRacingWheelDeadzone();

	// If the analog values haven't changed, don't bother sending any events for them
	const bool bHasDifferentAnalogInput = UE::GameInput::HasDifferentWheelAnalogInput(CurrentWheelState, PreviousState);
	if (!bHasDifferentAnalogInput)
	{
		return false;
	}

	OnControllerAnalog(Params, FGameInputKeys::RacingWheel_Brake.GetFName(), CurrentWheelState.brake, PreviousState.brake, Deadzone);
	OnControllerAnalog(Params, FGameInputKeys::RacingWheel_Clutch.GetFName(), CurrentWheelState.clutch, PreviousState.clutch, Deadzone);
	OnControllerAnalog(Params, FGameInputKeys::RacingWheel_Handbrake.GetFName(), CurrentWheelState.handbrake, PreviousState.handbrake, Deadzone);
	OnControllerAnalog(Params, FGameInputKeys::RacingWheel_Throttle.GetFName(), CurrentWheelState.throttle, PreviousState.throttle, Deadzone);
	OnControllerAnalog(Params, FGameInputKeys::RacingWheel_Wheel.GetFName(), CurrentWheelState.wheel, PreviousState.wheel, Deadzone);

	// Do we actually want this as a float? We should test the values that this can produce
	OnControllerAnalog(Params, 
		FGameInputKeys::RacingWheel_PatternShifterGear.GetFName(), 
		static_cast<float>(CurrentWheelState.patternShifterGear), 
		static_cast<float>(PreviousState.patternShifterGear), 
		Deadzone);

	// Keep track of the previous state
	PreviousState.brake = CurrentWheelState.brake;
	PreviousState.clutch = CurrentWheelState.clutch;
	PreviousState.handbrake = CurrentWheelState.handbrake;
	PreviousState.throttle = CurrentWheelState.throttle;
	PreviousState.wheel = CurrentWheelState.wheel;
	PreviousState.patternShifterGear = CurrentWheelState.patternShifterGear;

	return true;
}

bool FGameInputRacingWheelProcessor::ProcessWheelButtonState(const FGameInputEventParams& Params, GameInputRacingWheelState& CurrentWheelState)
{
	// If there has been no buttons pressed on this state or the previous one, don't bother trying
	// to evaluate any events.
	if (CurrentWheelState.buttons == 0 && PreviousState.buttons	== 0)
	{
		return false;
	}

	// This might not be necessary if the racing wheels also show up as "gamepad" devices...
	const uint32 CurrentButtonHeldMask = static_cast<uint32>(CurrentWheelState.buttons);

	uint32 LastButtonHeldMask = static_cast<uint32>(PreviousState.buttons);

	EvaluateButtonStates(
		Params,
		CurrentButtonHeldMask,
		OUT LastButtonHeldMask,
		RepeatTime,
		UE::GameInput::GetRacingWheelButtonMap(),
		MaxSupportedButtons);

	// Update the previous state here
	PreviousState.buttons = CurrentWheelState.buttons;

	return true;
}

GameInputKind FGameInputRacingWheelProcessor::GetSupportedReadingKind() const
{
	return GameInputKindRacingWheel;
}

#endif