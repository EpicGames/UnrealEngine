// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_Raw.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "InputCoreTypes.h"

#if GAME_INPUT_SUPPORT

#if UE_GAMEINPUT_SUPPORTS_RAW

constexpr float FGameInputRawDeviceProcessor::RawValueToFloatTrigger(const uint8 RawValue) const
{
	// Maps the uint8 value of 0-255 to a float between 0.0 and +1.0, like a gamepad trigger.
	return (static_cast<float>(RawValue) / 255.f);
}

constexpr float FGameInputRawDeviceProcessor::RawValueNibbleValueToFloat(const uint8 RawValue, const float Scale) const
{
	return (static_cast<float>(RawValue) / Scale);
}

const float FGameInputRawDeviceProcessor::RawValueToFloatAnalog(uint8 RawValue, const uint8 DeadZone /* = 2 */) const
{
	// Apply a simple square deadzone...
	{
		// Remember, we are mapping a uint8 (0-255) to a float of -1.0 and +1.0. So, any value
		// between 0 and 127 is negative and 129-255 is positive. 128 is the center.

		// Calculate the offset of how far this raw value is from center
		const int32 MaxOffset = FMath::Abs(RawValue - 128);

		// TODO: Implement a better deadzone then this, this is a square one
		// If we are within the deadzone, set this value to 128 which will translate to 0.0f
		if (MaxOffset <= DeadZone)
		{
			RawValue = 128;
		}
	}

	// Maps the uint8 value to a float between -1.0 and +1.0
	return ((static_cast<float>(RawValue) * (2.f / 255.f)) - 1.f);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const GameInputRawDeviceReportInfo* FGameInputRawDeviceProcessor::ReadCurrentRawInputState(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse)
{
	return Internal_AttemptToReadCurrentRawInputState(Params, ReadingToUse);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const GameInputRawDeviceReportInfo* FGameInputRawDeviceProcessor::Internal_AttemptToReadCurrentRawInputState(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse)
{
	// 	The GetReportInfo function signature doesn't exist in game input version 1 or 2, and changed in 3.0.
	// We still need this implementation for version 0 though, which is the GDK version of GameInput.
#if GAMEINPUT_API_VERSION <= 0
	
	if (!ReadingToUse)
	{
		UE_LOGF(LogGameInput, Error, "[ReadCurrentRawInputState] Cannot read raw input state, ReadingToUse was null (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
		return nullptr;
	}

	TComPtr<IGameInputRawDeviceReport> RawReport;
	const bool bSuccessfulReading = ReadingToUse->GetRawReport(&RawReport);
	if (!bSuccessfulReading)
	{
		UE_LOGF(LogGameInput, Error, "[ReadCurrentRawInputState] Unsuccessful reading of raw input report! (GetRawReport failed) (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
		return nullptr;
	}

	const int32 RawRepDataSize = RawReport->GetRawDataSize();

	// Ensure that the current data array is populated to the size needed.
	CurrentRawData.Reset();
	CurrentRawData.AddZeroed(RawRepDataSize);

	// Ensure that the previous data array is populated to the size needed so that we can compare its values per-index
	if (PreviousRawData.Num() < RawRepDataSize)
	{
		PreviousRawData.AddZeroed(RawRepDataSize - PreviousRawData.Num());
	}

	// This will populate the values in the CurrentRawData so that we can process them
	const int32 NumReadBytes = RawReport->GetRawData(RawReport->GetRawDataSize(), CurrentRawData.GetData());

	const GameInputRawDeviceReportInfo* RawReportInfo = RawReport->GetReportInfo();
	if (!RawReportInfo)
	{
		UE_LOGF(LogGameInput, Warning, "[ProcessRawReport] Unsuccessful reading of raw input report! (GameInputRawDeviceReportInfo is null) (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
	}

	return RawReportInfo;
	
#else
	ensureAlwaysMsgf(false, TEXT("Internal_AttemptToReadCurrentRawInputState should only be called for Game Input library version 0"));
	return nullptr;
#endif
}

GameInputRawDeviceReportInfo FGameInputRawDeviceProcessor::ReadCurrentRawInputStateInfo(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse)
{
	GameInputRawDeviceReportInfo RawReportInfo = {};

#if GAMEINPUT_API_VERSION >= 3
	if (!ReadingToUse)
	{
		UE_LOGF(LogGameInput, Error, "[ReadCurrentRawInputState] Cannot read raw input state, ReadingToUse was null (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
		return RawReportInfo;
	}

	TComPtr<IGameInputRawDeviceReport> RawReport;
	const bool bSuccessfulReading = ReadingToUse->GetRawReport(&RawReport);
	if (!bSuccessfulReading)
	{
		UE_LOGF(LogGameInput, Error, "[ReadCurrentRawInputState] Unsuccessful reading of raw input report! (GetRawReport failed) (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
		return RawReportInfo;
	}

	const int32 RawRepDataSize = RawReport->GetRawDataSize();

	// Ensure that the current data array is populated to the size needed.
	CurrentRawData.Reset();
	CurrentRawData.AddZeroed(RawRepDataSize);

	// Ensure that the previous data array is populated to the size needed so that we can compare its values per-index
	if (PreviousRawData.Num() < RawRepDataSize)
	{
		PreviousRawData.AddZeroed(RawRepDataSize - PreviousRawData.Num());
	}

	// This will populate the values in the CurrentRawData so that we can process them
	const int32 NumReadBytes = RawReport->GetRawData(RawReport->GetRawDataSize(), CurrentRawData.GetData());
	
	RawReport->GetReportInfo(&RawReportInfo);
	
#elif GAMEINPUT_API_VERSION <= 0

	// Older versions of the Game Input API at 0 or less can read raw data, but have different function signatures.
	// We can attempt to populate our raw data based on this if we can read it
	if (const GameInputRawDeviceReportInfo* InfoPtr = Internal_AttemptToReadCurrentRawInputState(Params, ReadingToUse))
	{
		RawReportInfo = *InfoPtr;
	}

#else
	UE_LOGF(LogGameInput, Error, "A GameInput version of 3+ is required for raw device support. (Device %ls) ", *UE::GameInput::LexToString(Params.Device));
#endif	// #if GAMEINPUT_API_VERSION >= 3
	
	return RawReportInfo;
}

bool FGameInputRawDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return bRes;
	}

	const FGameInputDeviceConfiguration* DeviceConfig = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo());

	// Check that we have a valid config before bothering to read the raw report
	if (!DeviceConfig)
	{
		UE_LOGF(LogGameInput, Verbose, "[ProcessRawReport] (Device %ls) Does not have a valid FGameInputDeviceConfiguration in the UGameInputDeveloperSettings. We can't process Raw Input without it. Exiting.", *UE::GameInput::LexToString(Params.Device));
		return bRes;
	}

	// Skip if this device config isn't even supposed to be processing raw input values
	if (!DeviceConfig->bProcessRawReportData)
	{
		return bRes;
	}

	// Read from the current reading the CURRENT frame
	const GameInputRawDeviceReportInfo RawReportInfo = ReadCurrentRawInputStateInfo(Params, Params.Reading);

	// Actually process the current raw input values if this reading ID matches the one we want
	if (RawReportInfo.id == DeviceConfig->RawReportReadingId)
	{
		// Process the BUTTON types here, we only want to process analog events once per frame to avoid over-accumulation
		constexpr bool bShouldProcessButtons = true;
		constexpr bool bShouldProcessAnalog = false;
		bRes |= ProcessAllRawValues(Params, DeviceConfig, bShouldProcessButtons, bShouldProcessAnalog);

		++NumReadingsProcessedThisFrame;
	}

	return bRes;
}


bool FGameInputRawDeviceProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	bool bRes = false;

	const bool bProcessedAnyGamepadButtonsThisFrame = (NumReadingsProcessedThisFrame > 0);

	// This is the last reading of this frame, reset the counter to 0
	NumReadingsProcessedThisFrame = 0;

	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return bRes;
	}

	const FGameInputDeviceConfiguration* DeviceConfig = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo());

	// Check that we have a valid config before bothering to read the raw report
	if (!DeviceConfig)
	{
		UE_LOGF(LogGameInput, Verbose, "[ProcessRawReport] (Device %ls) Does not have a valid FGameInputDeviceConfiguration in the UGameInputDeveloperSettings. We can't process Raw Input without it. Exiting.", *UE::GameInput::LexToString(Params.Device));
		return bRes;
	}

	// On the last input reading for the frame, the "Current Reading" should always be null. we only care for the LastReading here
	ensure(Params.Reading == nullptr);

	if (!Params.PreviousReading)
	{
		return bRes;
	}

	// Read from the current reading the PREVIOUS frame
	const GameInputRawDeviceReportInfo RawReportInfo = ReadCurrentRawInputStateInfo(Params, Params.PreviousReading);

	// Actually process the current raw input values if this reading ID matches the one we want
	if (RawReportInfo.id == DeviceConfig->RawReportReadingId)
	{
		// We always want to process analog events on the last frame
		constexpr bool bShouldPorcessAnalog = true;
		// We only need to process buttons this frame too if there have been no other readings yet.
		const bool bNeedToProcessButtons = !bProcessedAnyGamepadButtonsThisFrame;

		bRes |= ProcessAllRawValues(Params, DeviceConfig, bNeedToProcessButtons, bShouldPorcessAnalog);
	}

	// Track our previous input only on the last input frame. We don't want any duplicate readings
	PreviousRawData = CurrentRawData;

	return bRes;
}

bool FGameInputRawDeviceProcessor::ProcessAllRawValues(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* DeviceConfig, const bool bShouldProcessButtons, const bool bShouldProcessAnalog)
{
	ensure(CurrentRawData.Num() <= PreviousRawData.Num());

	ensure(Params.PlatformUserId.IsValid());

	bool bRes = false;

	for (int32 i = 0; i < CurrentRawData.Num(); ++i)
	{
		const uint8 Val = CurrentRawData[i];

		if (const FGameInputRawDeviceReportData* AxisData = DeviceConfig->RawReportMappingData.Find(i))
		{
			if (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsButtonBitmask)
			{
				if (bShouldProcessButtons)
				{
					bRes |= ProcessRawInputValueAsBitmask(Params, i, AxisData);
				}				
			}
			else if (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisPair)
			{
				// Only call this function on the higher index
				if (bShouldProcessAnalog && i == AxisData->HigherBitAxisIndex)
				{
					ProcessRawInputValueAsAanalogPaired(Params, AxisData);
				}
			}
			else if (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8)
			{
				if (AxisData->bTreatValuesAsAnalog && bShouldProcessAnalog || (!AxisData->bTreatValuesAsAnalog && bShouldProcessButtons))
				{
					bRes |= ProcessRawInputValuePackedUInt8(Params, i, AxisData);	
				}
			}
			// All other methods require a valid key name on the config
			else if (AxisData->KeyName.IsValid())
			{
				// Treat this value as a button. If it is non-zero then consider it pressed. If it is zero, then it is not pressed.
				if (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsButton)
				{
					if (bShouldProcessButtons)
					{
						bRes |= ProcessRawInputValueAsButton(Params, i, AxisData);
					}					
				}
				// Otherwise we can do analog values, which can be either "trigger" or "analog" types
				else if (bShouldProcessAnalog)
				{
					bRes |= ProcessRawInputValueAsAanalog(Params, i, AxisData);
				}
			}
			else
			{
				// You want a valid key name here, throw a warning if your config is wrong
				UE_LOGF(LogGameInput, Warning, "[ProcessRawReport] Invalid key name for raw report axis at index %d with value of %u (Device %ls)", i, Val, *UE::GameInput::LexToString(Params.Device));
			}
		}
		else if (Val > 0)
		{
			UE_LOGF(LogGameInput, VeryVerbose, "[ProcessRawReport] (Device %ls) No raw device report config for axis '%d' with value of %u", *UE::GameInput::LexToString(Params.Device), i, Val);
		}
	}

	return bRes;
}

bool FGameInputRawDeviceProcessor::ProcessRawInputValueAsBitmask(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData)
{
	check(AxisData && AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsButtonBitmask);

	const uint8 Val = CurrentRawData[RawValueIndex];
	const uint8 PrevVal = PreviousRawData[RawValueIndex];

	// Ensure we have a compatible key map setup
	FPerRawInputIndexData& IndexData = RawInputIndexDataMap.FindOrAdd(RawValueIndex);

	if (IndexData.KeyNameMap.IsEmpty())
	{
		// The settings use their button map as a bit number, so the key map we actually need to use
		// is 1 << that bit to have EvaluateButtonStates work correctly
		for (const TPair<int32, FName>& MappingPair : AxisData->ButtonBitMaskMappings)
		{
			IndexData.KeyNameMap.Add({ 1 << MappingPair.Key, MappingPair.Value });
		}
	}

	const uint32 CurrentValue32 = static_cast<uint32>(Val);
	uint32 PreviousValue32 = static_cast<uint32>(PrevVal);

	// Note: Set the max supported buttons here to 1 because we only care about the first bit
	EvaluateButtonStates(
		Params,
		CurrentValue32,
		OUT PreviousValue32,
		IndexData.RepeatTime,
		IndexData.KeyNameMap,
		FPerRawInputIndexData::MaxSupportedButtons);

	// If this value is non-zero then we had a reading
	return Val != 0;
}

bool FGameInputRawDeviceProcessor::ProcessRawInputValueAsButton(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData)
{
	check(AxisData && AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsButton);

	const uint8 Val = CurrentRawData[RawValueIndex];
	const uint8 PrevVal = PreviousRawData[RawValueIndex];

	// Just treat this button as pressed when non-zero, and not pressed when 0.
	const uint32 CurrentValue32 = FMath::Clamp<uint32>(static_cast<uint32>(Val), 0, 1);
	uint32 PreviousValue32 = FMath::Clamp<uint32>(static_cast<uint32>(PrevVal), 0, 1);

	// Map the value of the key name to this 1 if we haven't already
	FPerRawInputIndexData& IndexData = RawInputIndexDataMap.FindOrAdd(RawValueIndex);
	if (IndexData.KeyNameMap.IsEmpty())
	{
		IndexData.KeyNameMap.Add(1, AxisData->KeyName);
	}

	// Note: Set the max supported buttons here to 1 because we only care about the first bit
	EvaluateButtonStates(
		Params,
		CurrentValue32,
		OUT PreviousValue32,
		IndexData.RepeatTime,
		IndexData.KeyNameMap,
		/* maxSupportedButtons */ 1);

	// If this value is non-zero then we had a reading
	return Val != 0;
}

bool FGameInputRawDeviceProcessor::ProcessRawInputValueAsAanalog(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData)
{
	check(AxisData);

	const uint8 Val = CurrentRawData[RawValueIndex];
	const uint8 PrevVal = PreviousRawData[RawValueIndex];

	const float CurrentValueFloat = AxisData->Scalar * (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsTrigger ? RawValueToFloatTrigger(Val) : RawValueToFloatAnalog(Val, AxisData->AnalogDeadzone));
	const float PreviousValueFloat = AxisData->Scalar * (AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsTrigger ? RawValueToFloatTrigger(PrevVal) : RawValueToFloatAnalog(PrevVal, AxisData->AnalogDeadzone));
	
	// A valid FKey is required
	if (!FKey(AxisData->KeyName).IsValid())
	{
		UE_LOGF(LogGameInput, Error, "Attempting to report an invalid key name '%ls' (Device %ls) ", *AxisData->KeyName.ToString(), *UE::GameInput::LexToString(Params.Device));
		return false;
	}

	OnControllerAnalog(
		Params, 
		AxisData->KeyName, 
		CurrentValueFloat, 
		PreviousValueFloat, 
		UE::GameInput::GamepadLeftStickDeadzone, 
		/* bShouldSetDeviceScope = */!AxisData->bIgnoreAnalogInputDeviceScopeForThisRawReport);

	// We had a reading as long as it is non-zero
	return CurrentValueFloat != 0.0f;
}

bool FGameInputRawDeviceProcessor::ProcessRawInputValueAsAanalogPaired(const FGameInputEventParams& Params, const FGameInputRawDeviceReportData* AxisData)
{
	check(AxisData);

	if (!CurrentRawData.IsValidIndex(AxisData->LowerBitAxisIndex) || !CurrentRawData.IsValidIndex(AxisData->HigherBitAxisIndex))
	{
		return false;
	}

	// Get the current value
	const uint8 CurrentLowerVal = CurrentRawData[AxisData->LowerBitAxisIndex];
	const uint8 CurrentHigherVal = CurrentRawData[AxisData->HigherBitAxisIndex];
	
	// Combine the two values into a single int16. Do this by 
	// shifting the higher value up by 8 bits, and then just use the lower value in our int16	
	const int16 CurrentPackedVal = (int16)((CurrentHigherVal << 8) | CurrentLowerVal);
	const float CurrentValueFloat = (static_cast<float>(CurrentPackedVal) / 32767.f);

	// Get the previous value
	const uint8 PreviousLowerVal = PreviousRawData[AxisData->LowerBitAxisIndex];
	const uint8 PreviousHigherVal = PreviousRawData[AxisData->HigherBitAxisIndex];

	const int16 PreviousPackedVal = (int16)((PreviousHigherVal << 8) | PreviousLowerVal);
	const float PreviousValueFloat = (static_cast<float>(PreviousPackedVal) / 32767.f);

	if (!AxisData->KeyName.IsValid())
	{
		return false;
	}

	OnControllerAnalog(Params, AxisData->KeyName, CurrentValueFloat, PreviousValueFloat, UE::GameInput::GamepadLeftStickDeadzone);

	// We had a reading as long as it is non-zero
	return CurrentValueFloat != 0.0f;
}

bool FGameInputRawDeviceProcessor::ProcessRawInputValuePackedUInt8(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData)
{
	check(AxisData && AxisData->TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8);

	// Isolate the lower bits as a uint8 value (masking off the upper 4, and only using the lower 4)
	const uint8 CurrentLowerBitValues = CurrentRawData[RawValueIndex] & 0x0F;
	const uint8 PreviousLowerBitValues = PreviousRawData[RawValueIndex] & 0x0F;
	
	// To get the upper 4 bits as a uint8 value, we can just shift them right 4 and drop off the lower 4 values.
	const uint8 CurrentUpperBitValues = CurrentRawData[RawValueIndex] >> 4;
	const uint8 PreviousUpperBitValues = PreviousRawData[RawValueIndex] >> 4;
	
	// Map the value of the key name to this 1 if we haven't already, to keep track of the button repeat timings
	FPerRawInputIndexData& IndexData = RawInputIndexDataMap.FindOrAdd(RawValueIndex);

	// Report these values as analog data
	if (AxisData->bTreatValuesAsAnalog)
	{
		EvaluatePackedUInt8ButtonChordsAsAnalog(Params, CurrentLowerBitValues, PreviousLowerBitValues, AxisData->LowerBitKeys);
    	EvaluatePackedUInt8ButtonChordsAsAnalog(Params, CurrentUpperBitValues, PreviousUpperBitValues, AxisData->UpperBitKeys);	
	}
	// Otherwise, report them as key press events
	else
	{
		static constexpr int32 LowerBitsRepeatTimeIndex = 0;
		static constexpr int32 UpperBitsRepeatTimeIndex = 1;
	

		// Notice: We are using the !! operator here because we want to only send values of 0 or 1 for button presses
		// to avoid needlessly over reporting. For non-analog data, anything that is non-zero will be considered a "pressed" state
		EvaluatePackedUInt8ButtonChordsAsButtonEvents(Params, !!CurrentLowerBitValues, !!PreviousLowerBitValues, &IndexData.RepeatTime[LowerBitsRepeatTimeIndex], AxisData->LowerBitKeys);
		EvaluatePackedUInt8ButtonChordsAsButtonEvents(Params, !!CurrentUpperBitValues, !!PreviousUpperBitValues, &IndexData.RepeatTime[UpperBitsRepeatTimeIndex], AxisData->UpperBitKeys);
	}

	// If the upper or lower value is non-zero then we had a reading
	return CurrentLowerBitValues != 0 || CurrentUpperBitValues != 0;
}

void FGameInputRawDeviceProcessor::EvaluatePackedUInt8ButtonChordsAsAnalog(
	const FGameInputEventParams& Params,
	const uint8 CurrentValue,
	const uint8 PreviousValue,
	const TArray<FGamepadKeyNames::Type>& ButtonNames)
{
	// We can't tell slate about input messages from an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		UE_LOGF(LogGameInput, Verbose, "Attempting to evaluate button states with an invalid platform user id of '%d'. The button messages will not be sent.", Params.PlatformUserId.GetInternalId());
		return;
	}

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInputDeviceScope InputScope(
		nullptr,
		UE::GameInput::GetDeviceScopeClassName(),
		DeviceMapper.GetUserIndexForPlatformUser(Params.PlatformUserId),
		FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (CurrentValue != PreviousValue)
	{
		const float CurrentValueFloat = RawValueNibbleValueToFloat(CurrentValue);

		// Report an analog value for each of the buttons that we want
		for (const FGamepadKeyNames::Type& KeyName : ButtonNames)
		{
			if (!FKey(KeyName).IsValid())
			{
				continue;
			}
			
			UE_LOGF(LogGameInput, Verbose, "[FGameInputRawDeviceProcessor::EvaluatePackedUInt8ButtonChordsAsAnalog] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Value %.3f",
				Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString(), CurrentValueFloat);
			
			Params.MessageHandler->OnControllerAnalog(KeyName, Params.PlatformUserId, Params.InputDeviceId, CurrentValueFloat);	
		}
	}
}

void FGameInputRawDeviceProcessor::EvaluatePackedUInt8ButtonChordsAsButtonEvents(
	const FGameInputEventParams& Params,
	const uint8 CurrentValue,
	const uint8 PreviousValue,
	double* RepeatTime,
	const TArray<FGamepadKeyNames::Type>& ButtonNames)
{
	// We can't tell slate about input messages from an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		UE_LOGF(LogGameInput, Verbose, "Attempting to evaluate button states with an invalid platform user id of '%d'. The button messages will not be sent.", Params.PlatformUserId.GetInternalId());
		return;
	}

	if (!RepeatTime)
	{
		UE_LOGF(LogGameInput, Error, "[IGameInputDeviceProcessor::EvaluateButtonStates] Invalid RepeatTime array given to evaluate button states!");
		return;
	}

	if (ButtonNames.IsEmpty())
	{
		return;
	}

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FInputDeviceScope InputScope(nullptr, FName("GameInput"), DeviceMapper.GetUserIndexForPlatformUser(Params.PlatformUserId), FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	auto ForEachValidKey = [&ButtonNames](TFunction<void(const FGamepadKeyNames::Type&)>&& Predicate)
	{
		for (const FGamepadKeyNames::Type& KeyName : ButtonNames)
		{
			if (!FKey(KeyName).IsValid())
			{
				continue;
			}
			
			Predicate(KeyName);
		}
	};

	const double CurrentTime = FPlatformTime::Seconds();

	// Handle the pressing or releasing of a key (the value has changed)
	if (CurrentValue != PreviousValue)
	{
		if (CurrentValue)
		{
			ForEachValidKey([&Params](const FGamepadKeyNames::Type& KeyName)
			{
				UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Pressed",
					Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
					
				Params.MessageHandler->OnControllerButtonPressed(KeyName, Params.PlatformUserId, Params.InputDeviceId, false);
			});

			*RepeatTime = CurrentTime + InitialButtonRepeatDelay;
		}
		else
		{
			ForEachValidKey([&Params](const FGamepadKeyNames::Type& KeyName)
			{
				UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Released",
					Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
					
				Params.MessageHandler->OnControllerButtonReleased(KeyName, Params.PlatformUserId, Params.InputDeviceId, false);
			});

			*RepeatTime = 0.0;
		}
	}

	// Handle button repeat events
	if (CurrentValue && CurrentValue == PreviousValue && CurrentTime > *RepeatTime)
	{
		*RepeatTime = CurrentTime + ButtonRepeatDelay;
		
		ForEachValidKey([&Params](const FGamepadKeyNames::Type& KeyName)
		{
			UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Repeat Pressed",
				Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
				
			Params.MessageHandler->OnControllerButtonPressed(KeyName, Params.PlatformUserId, Params.InputDeviceId, true);
		});
	}
}

void FGameInputRawDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return;
	}

	const FGameInputDeviceConfiguration* DeviceConfig = GetDefault<UGameInputDeveloperSettings>()->FindDeviceConfiguration(Params.GetDeviceInfo());

	// Check that we have a valid config before bothering to read the raw report
	if (!DeviceConfig)
	{
		UE_LOGF(LogGameInput, Verbose, "[ClearStateRawReport] No have a valid FGameInputDeviceConfiguration in the UGameInputDeveloperSettings. We can't process Raw Input without it. Exiting. (Device %ls)", *UE::GameInput::LexToString(Params.Device));
		return;
	}

	if (!DeviceConfig->bProcessRawReportData)
	{
		return;
	}

	// Reset our current values to 0...
	for (int32 i = 0; i < CurrentRawData.Num(); ++i)
	{
		CurrentRawData[i] = 0;
	}

	// ... and then process the raw values as if there is 0 input. We want to process all types when clearing.
	constexpr bool bShouldPorcessButtons = true;
	constexpr bool bShouldPorcessAnalog = true;

	ProcessAllRawValues(Params, DeviceConfig, bShouldPorcessButtons, bShouldPorcessAnalog);
}

GameInputKind FGameInputRawDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindRawDeviceReport;
}

#endif	// UE_GAMEINPUT_SUPPORTS_RAW

#endif	// #if GAME_INPUT_SUPPORT
