// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_Sensor.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "GameInputKeyTypes.h"

#if GAME_INPUT_SUPPORT

#if UE_GAMEINPUT_SUPPORTS_SENSORS

bool FGameInputGamepadSensorDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	// Accumulate (potentially) multiple samples per tick

	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid() || !Params.PreviousReading)
	{
		return false;
	}

	GameInputSensorsState SensorState = {};
	if (!Params.PreviousReading->GetSensorsState(&SensorState))
	{
		return false;
	}

	const GameInputDeviceInfo* Info = Params.GetDeviceInfo();
	if (!Info || !Info->sensorsInfo)
	{
		return false;
	}

	const GameInputSensorsKind DetectedSensorKind = Info->sensorsInfo->supportedSensors;

	// This device does not support any sensors. Don't bother trying to get a reading from it.
	if (DetectedSensorKind == GameInputSensorsNone)
	{
		return false;
	}

	static constexpr GameInputSensorsKind RequiredMotionDataKinds = (GameInputSensorsGyrometer | GameInputSensorsAccelerometer);

	// Orientation, accel, and gyro data 
	if (!(DetectedSensorKind & RequiredMotionDataKinds))
	{
		return false;
	}

	if (NumReadingsProcessedThisFrame <= 0)
	{
		FMemory::Memset(AccumulatedState, 0);
		NumReadingsProcessedThisFrame = 0;
	}

	// Acceleration: we want the average of all samples from this tick
	AccumulatedState.accelerationInGX += SensorState.accelerationInGX;
	AccumulatedState.accelerationInGY += SensorState.accelerationInGY;
	AccumulatedState.accelerationInGZ += SensorState.accelerationInGZ;

	// Angular velocity: we want the average of all samples from this tick
	AccumulatedState.angularVelocityInRadPerSecX += SensorState.angularVelocityInRadPerSecX;
	AccumulatedState.angularVelocityInRadPerSecY += SensorState.angularVelocityInRadPerSecY;
	AccumulatedState.angularVelocityInRadPerSecZ += SensorState.angularVelocityInRadPerSecZ;

	// Orientation: we just want the latest
	AccumulatedState.orientationW = SensorState.orientationW;
	AccumulatedState.orientationX = SensorState.orientationX;
	AccumulatedState.orientationY = SensorState.orientationY;
	AccumulatedState.orientationZ = SensorState.orientationZ;

	// Heading: we just want the latest
	AccumulatedState.headingInDegreesFromMagneticNorth = SensorState.headingInDegreesFromMagneticNorth;
	AccumulatedState.headingAccuracy = SensorState.headingAccuracy;

	++NumReadingsProcessedThisFrame;

	return true;
}

bool FGameInputGamepadSensorDeviceProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	if (!ProcessInput(Params) && NumReadingsProcessedThisFrame == 0)
	{
		return false;
	}

	if (NumReadingsProcessedThisFrame > 1)
	{
		// More than one reading means we have to convert some of our accumulated inputs into averages
		const float AverageScalar = 1.f / static_cast<float>(NumReadingsProcessedThisFrame);

		// Acceleration:
		AccumulatedState.accelerationInGX *= AverageScalar;
		AccumulatedState.accelerationInGY *= AverageScalar;
		AccumulatedState.accelerationInGZ *= AverageScalar;

		// Angular velocity:
		AccumulatedState.angularVelocityInRadPerSecX *= AverageScalar;
		AccumulatedState.angularVelocityInRadPerSecY *= AverageScalar;
		AccumulatedState.angularVelocityInRadPerSecZ *= AverageScalar;
	}

	// Negative values say "we've not yet processed any readings THIS frame, but we HAVE had readings before".
	// In that case, the negative value tells us how many more times we should repeat the same report.
	// Repeating the same report for a few frames is necessary when game framerate exceeds controller report rate and can also help with small hitches when the controller connection is poor.
	if (NumReadingsProcessedThisFrame < 0)
	{
		++NumReadingsProcessedThisFrame;
	}
	else
	{
		static constexpr int32 ReadingRequiresRepeatReport = -30;
		NumReadingsProcessedThisFrame = ReadingRequiresRepeatReport;
	}

	const bool bHasSensorReading = ProcessGamepadSensorState(Params, AccumulatedState);

	PreviousState = AccumulatedState;

	return bHasSensorReading;
}

void FGameInputGamepadSensorDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// Can't do anything for an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		return;
	}

	// We can simply process a sensor state where everything is zero
	FMemory::Memset(AccumulatedState, 0);
	NumReadingsProcessedThisFrame = 0;

	ProcessGamepadSensorState(Params, AccumulatedState);

	// Make sure to update the previous state here
	PreviousState = AccumulatedState;
}

bool FGameInputGamepadSensorDeviceProcessor::ProcessGamepadSensorState(const FGameInputEventParams& Params, GameInputSensorsState& SensorState)
{
	const GameInputDeviceInfo* Info = Params.GetDeviceInfo();
	if (!Info || !Info->sensorsInfo)
	{
		return false;
	}
	
	const GameInputSensorsKind DetectedSensorKind = Info->sensorsInfo->supportedSensors;
	
	// This device does not support any sensors. Don't bother trying to get a reading from it.
	if (DetectedSensorKind == GameInputSensorsNone)
	{
		return false;
	}

	bool bHasSensorReading = false;
	
	static constexpr GameInputSensorsKind RequiredMotionDataKinds = (GameInputSensorsGyrometer | GameInputSensorsAccelerometer);
	
	// Orientation, accel, and gyro data 
	if (DetectedSensorKind & RequiredMotionDataKinds)
	{
		const FVector AngularVelocity = { SensorState.angularVelocityInRadPerSecX, SensorState.angularVelocityInRadPerSecY, SensorState.angularVelocityInRadPerSecZ };
		const FVector Accel = { SensorState.accelerationInGX, SensorState.accelerationInGY, SensorState.accelerationInGZ };
		
		const FQuat Orientation(SensorState.orientationW, SensorState.orientationX, SensorState.orientationY, SensorState.orientationZ);
		const FRotator Rotation = Orientation.Rotator();

		// TODO_BH: Unify motion space settings. This is using the "1" setting from other platforms 
		
		Params.MessageHandler->OnMotionDetected(
				FVector(Rotation.Pitch, Rotation.Yaw, Rotation.Roll),
				FVector(AngularVelocity.Z, -AngularVelocity.X, -AngularVelocity.Y),
				FVector::ZeroVector,
				FVector(-Accel.Z, Accel.X, Accel.Y),
				Params.PlatformUserId,
				Params.InputDeviceId
			);

		UE_LOGF(LogGameInput, VeryVerbose, "[%hs] OnMotionDetected PlatUser: %d DeviceId %d  MotionSpace: %d",
			__func__,
			Params.PlatformUserId.GetInternalId(),
			Params.InputDeviceId.GetId(),
			1);

		bHasSensorReading = true;
	}

	if (DetectedSensorKind & GameInputSensorsCompass)
	{
		// TODO_BH: We have SensorState.headingAccuracy, so perhaps we can have a different deadzone setting for different accuracies.
		OnControllerAnalog(
			Params,
			FGameInputKeys::Sensor_HeadingInDegreesFromMagneticNorth.GetFName(),
			SensorState.headingInDegreesFromMagneticNorth,
			PreviousState.headingInDegreesFromMagneticNorth,
			UE::GameInput::SensorMagneticNorthDeadzone);

		UE_LOGF(LogGameInput, VeryVerbose, "[ProcessGamepadSensorState] (Device %ls) Degrees From Magnetic North: %.2f (Accuracy: %ls) ",
			*UE::GameInput::LexToString(Params.Device), SensorState.headingInDegreesFromMagneticNorth, *UE::GameInput::LexToString(SensorState.headingAccuracy));
	}
	
	return bHasSensorReading;
}

GameInputKind FGameInputGamepadSensorDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindSensors;
}

#endif	// #if UE_GAMEINPUT_SUPPORTS_SENSORS

#endif	// #if GAME_INPUT_SUPPORT