// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/GameInputDeviceProcessor_Touch.h"

#include "Framework/Application/SlateApplication.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "HAL/ConsoleManager.h"

#if GAME_INPUT_SUPPORT

namespace GameInputDevicePreProcessorCVars
{
	bool bEnableTouchPressureCorrection = false;
	FAutoConsoleVariableRef CVarEnableTouchPressureCorrection(
		TEXT("GameInputDeviceProcessor.EnableTouchPressureCorrection"),
		bEnableTouchPressureCorrection,
		TEXT("This enables overriding touch input pressure in case the readings received by the touch input device preprocessor is considered too small."));

	float TouchPressureCorrectionValue = 1.f;
	FAutoConsoleVariableRef CVarTouchPressureCorrectionValue(
		TEXT("GameInputDeviceProcessor.TouchPressureCorrectionValue"),
		TouchPressureCorrectionValue,
		TEXT("Touch pressure value to be used in case the touch input device preprocessor receives a reading that is considered to be too small"));
}; // namespace GameInputDevicePreProcessorCVars

#if UE_GAMEINPUT_SUPPORTS_TOUCH



bool FGameInputTouchDeviceProcessor::ProcessInput(const FGameInputEventParams& Params)
{
	// read the new touch events
	int32 TouchCount = Params.Reading->GetTouchCount();
	TArray<GameInputTouchState> InputTouchData;
	InputTouchData.SetNum(TouchCount, EAllowShrinking::No);
	Params.Reading->GetTouchState(TouchCount, InputTouchData.GetData());

	// no new touch events and we dont have any previous inputs
	if (TouchCount == 0 && ActiveTouchPoints == 0)
	{
		return false;
	}

	// We can't tell slate about input messages from an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		UE_LOGF(LogGameInput, Verbose, "[FGameInputTouchDeviceProcessor::EvaluateButtonStates] Attempting to evaluate button states with an invalid platform user id of '%d'. The button messages will not be sent.", Params.PlatformUserId.GetInternalId());
		return false;
	}

	auto FindOrAddTouch = [](TArray<FTouchData>& TouchStates, const GameInputTouchState& NewTouchData) -> int32
	{
		int32 Index = TouchStates.IndexOfByPredicate([&](const FTouchData& A) {return A.TouchId == NewTouchData.touchId; });
		if (Index == INDEX_NONE)
		{
			// check if we have a free slot
			Index = TouchStates.IndexOfByPredicate([&](const FTouchData& A) {return A.TouchId == INDEX_NONE; });

			// no free slot - create a new one. (NB. We will never receive more than GameInputDeviceInfo::touchPointCount separate touches)
			if (Index == INDEX_NONE)
			{
				Index = TouchStates.Add(FTouchData());
			}
		}
		return Index;
	};

	// Reset the active touch state from the previous frame
	for (FTouchData& TouchData : PreviousTouchData)
	{
		TouchData.bIsActive = false;
	}

	// process all previous frame events and check if there are in list of new events for this frame
	for (const GameInputTouchState& InputTouch : InputTouchData)
	{
		// search for the same touchId in the new events list
		int32 Index = FindOrAddTouch(PreviousTouchData, InputTouch);

		if (Index == INDEX_NONE)
		{
			continue;
		}

		FTouchData& TouchState = PreviousTouchData[Index];

		// need to scale by resolution
		FVector2D NewPosition = FVector2D(InputTouch.positionX * (float)MaxTouchX, InputTouch.positionY * (float)MaxTouchY).RoundToVector();

		// reading may fail to have a proper nonzero touch pressure, so we correct it
		const bool bShouldCorrectTouchPressure = GameInputDevicePreProcessorCVars::bEnableTouchPressureCorrection && FMath::Abs(InputTouch.pressure) < GameInputDevicePreProcessorCVars::TouchPressureCorrectionValue;
		const float NewTouchPressure = bShouldCorrectTouchPressure ? GameInputDevicePreProcessorCVars::TouchPressureCorrectionValue : InputTouch.pressure;

		if (TouchState.TouchId == INDEX_NONE)
		{
			Params.MessageHandler->OnTouchStarted(nullptr, NewPosition, NewTouchPressure, Index, Params.PlatformUserId, Params.InputDeviceId);
			TouchState.TouchId = InputTouch.touchId;
			ActiveTouchPoints++;
		}
		else if (NewPosition != TouchState.Position)
		{
			if (!TouchState.bHasMoved)
			{
				Params.MessageHandler->OnTouchFirstMove(NewPosition, NewTouchPressure, Index, Params.PlatformUserId, Params.InputDeviceId);
				TouchState.bHasMoved = true;
			}
			else
			{
				Params.MessageHandler->OnTouchMoved(NewPosition, NewTouchPressure, Index, Params.PlatformUserId, Params.InputDeviceId);
			}
		}

		if (TouchState.Pressure != NewTouchPressure)
		{
			Params.MessageHandler->OnTouchForceChanged(NewPosition, NewTouchPressure, Index, Params.PlatformUserId, Params.InputDeviceId);
		}

		TouchState.Pressure = NewTouchPressure;
		TouchState.Position = NewPosition;
		TouchState.bIsActive = true;
	}

	// process all new event that where not in the stack
	for (int32 Index = 0; Index < PreviousTouchData.Num(); Index++)
	{
		FTouchData& TouchData = PreviousTouchData[Index];
		if (!TouchData.bIsActive && TouchData.TouchId != INDEX_NONE)
		{
			Params.MessageHandler->OnTouchEnded(TouchData.Position, Index, Params.PlatformUserId, Params.InputDeviceId);
			TouchData = FTouchData();
			ActiveTouchPoints--;
		}
	}

	check(ActiveTouchPoints >= 0);

	return true;
}

void FGameInputTouchDeviceProcessor::ClearState(const FGameInputEventParams& Params)
{
	// There is nothing to be done for touch data clearing...
}

GameInputKind FGameInputTouchDeviceProcessor::GetSupportedReadingKind() const
{
	return GameInputKindTouch;
}

#endif	// #if UE_GAMEINPUT_SUPPORTS_TOUCH

#endif	// GAME_INPUT_SUPPORT