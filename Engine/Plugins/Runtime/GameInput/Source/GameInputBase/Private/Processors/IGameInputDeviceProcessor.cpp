// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/IGameInputDeviceProcessor.h"

#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "GameInputUtils.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "HAL/PlatformTime.h"
#include "Misc/ConfigCacheIni.h"

#if GAME_INPUT_SUPPORT

IGameInputDeviceProcessor::IGameInputDeviceProcessor()
{
	InitialButtonRepeatDelay = UE::GameInput::InitialRepeatDelay;
	ButtonRepeatDelay = UE::GameInput::SubsequentRepeatDelay;

	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("InitialButtonRepeatDelay"), InitialButtonRepeatDelay, GInputIni);
	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("ButtonRepeatDelay"), ButtonRepeatDelay, GInputIni);
}

const GameInputDeviceInfo* IGameInputDeviceProcessor::FGameInputEventParams::GetDeviceInfo() const
{
	return UE::GameInput::GetDeviceInfo(Device);
}

bool IGameInputDeviceProcessor::PostProcessInput(const FGameInputEventParams& Params)
{
	// Nothing needs to be done by default here in PostProcessInput.
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FString IGameInputDeviceProcessor::GetHardwareDeviceIdentifierName(const IGameInputDeviceProcessor::FGameInputEventParams& Params) const
{
	return FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void IGameInputDeviceProcessor::OnControllerAnalog(const FGameInputEventParams& Params, const FName& GamePadKey, float NewAxisValueNormalized, float OldAxisValueNormalized, float DeadZone, const bool bSetDeviceScope /*= true*/)
{
	if (OldAxisValueNormalized != NewAxisValueNormalized || FMath::Abs(NewAxisValueNormalized) > DeadZone)
	{		
		UE_LOGF(LogGameInput, VeryVerbose, "Device %ls (PlatformUserId = %d, InputDeviceId = %d) - Analog %ls : %.3f",
			*UE::GameInput::LexToString(Params.Device),
			Params.PlatformUserId.GetInternalId(),
			Params.InputDeviceId.GetId(),
			*GamePadKey.ToString(),
			NewAxisValueNormalized);

		// We should only tell slate about this message if the platform user and input device are valid because it will attempt
		// to create a new slate user based on the index if it doesn't already exist
		if (Params.PlatformUserId.IsValid() && Params.InputDeviceId.IsValid())
		{
			if (bSetDeviceScope)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FInputDeviceScope InputScope(
					nullptr,
					UE::GameInput::GetDeviceScopeClassName(),
					IPlatformInputDeviceMapper::Get().GetUserIndexForPlatformUser(Params.PlatformUserId),
					FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				Params.MessageHandler->OnControllerAnalog(GamePadKey, Params.PlatformUserId, Params.InputDeviceId, NewAxisValueNormalized);
			}
			else
			{
				Params.MessageHandler->OnControllerAnalog(GamePadKey, Params.PlatformUserId, Params.InputDeviceId, NewAxisValueNormalized);
			}
		}		
	}
}

void IGameInputDeviceProcessor::EvaluateButtonStates(
	const FGameInputEventParams& Params,
	const uint32 CurrentButtonHeldMask,
	uint32& PreviousButtonMask,
	double* RepeatTime,
	const TMap<uint32, FGamepadKeyNames::Type>& UnrealButtonNameMap,
	const uint32 SupportedButtonCount /*= MaxSupportedButtons*/)
{
	// handle button change events
	const uint32 ActionMask = (PreviousButtonMask ^ CurrentButtonHeldMask);
	const uint32 RepeatMask = (PreviousButtonMask & CurrentButtonHeldMask);
	uint32 BitMask = 1;

	if (!RepeatTime)
	{
		UE_LOGF(LogGameInput, Error, "[IGameInputDeviceProcessor::EvaluateButtonStates] Invalid RepeatTime array given to evaluate button states!");
		return;
	}

	// If the given button mask and repeat mask are both zero, then no buttons have been pressed or had a state change. No need to iterate the bitmask
	if (ActionMask == 0 && RepeatMask == 0)
	{
		return;
	}

	// We can't tell slate about input messages from an invalid platform user
	if (!Params.PlatformUserId.IsValid())
	{
		UE_LOGF(LogGameInput, Verbose, "[IGameInputDeviceProcessor::EvaluateButtonStates] Attempting to evaluate button states with an invalid platform user id of '%d'. The button messages will not be sent.", Params.PlatformUserId.GetInternalId());
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

	for (uint32 n = 0; n < SupportedButtonCount; ++n)
	{
		FGamepadKeyNames::Type ButtonKey;
		if (UE::GameInput::GameInputButtonToUnrealName(UnrealButtonNameMap, BitMask, ButtonKey))
		{
			// Check for button state change
			if (0 != (ActionMask & BitMask))
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FInputDeviceScope InputScope(nullptr, UE::GameInput::GetDeviceScopeClassName(), DeviceMapper.GetUserIndexForPlatformUser(Params.PlatformUserId), FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (0 != (CurrentButtonHeldMask & BitMask))
				{
					UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Pressed", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *ButtonKey.ToString());
					Params.MessageHandler->OnControllerButtonPressed(ButtonKey, Params.PlatformUserId, Params.InputDeviceId, false);
					RepeatTime[n] = CurrentTime + InitialButtonRepeatDelay;
				}
				else
				{
					UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Released", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *ButtonKey.ToString());
					Params.MessageHandler->OnControllerButtonReleased(ButtonKey, Params.PlatformUserId, Params.InputDeviceId, false);
					RepeatTime[n] = 0.0;
				}
			}

			// Check for repeat key
			if (0 != (RepeatMask & BitMask) && CurrentTime > RepeatTime[n])
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FInputDeviceScope InputScope(nullptr, UE::GameInput::GetDeviceScopeClassName(), DeviceMapper.GetUserIndexForPlatformUser(Params.PlatformUserId), FString(UE::GameInput::GetHardwareDeviceIdentifierName(Params.Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				RepeatTime[n] = CurrentTime + ButtonRepeatDelay;
				UE_LOGF(LogGameInput, Verbose, "[FGameInputDevice::EvaluateButtonStates] (PlatformUserId = %d, InputDeviceId = %d) - Button '%ls' Repeat Pressed", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *ButtonKey.ToString());
				Params.MessageHandler->OnControllerButtonPressed(ButtonKey, Params.PlatformUserId, Params.InputDeviceId, true);
			}
		}

		// Move on to the next bit!
		BitMask <<= 1;
	}

	PreviousButtonMask = CurrentButtonHeldMask;
}

void IGameInputDeviceProcessor::EvaluateSwitchState(
	const FGameInputEventParams& Params,
	GameInputSwitchPosition CurrentPosition,
	GameInputSwitchPosition& PreviousPosition,
	TArray<double>& RepeatTimes)
{
	if (!ensureMsgf(RepeatTimes.IsValidIndex(static_cast<uint32>(GameInputSwitchLeft)), TEXT("RepeatTimes array needs to be the same size as the number of switch positions!")))
	{
		return;
	}
	GameInputSwitchPosition PrevCopy = PreviousPosition;

	// If the current and previous switch states are both the center, then nothing has happened.
	if (CurrentPosition == GameInputSwitchCenter && PreviousPosition == GameInputSwitchCenter)
	{
		return;
	}

	// We can't send any slate input events to slate if the platform user is invalid
	if (!Params.PlatformUserId.IsValid())
	{
		UE_LOGF(LogGameInput, Verbose, "[IGameInputDeviceProcessor::EvaluateSwitchState] Attempting to evaluate button states with an invalid platform user id of '%d'. The input messages will not be sent.", Params.PlatformUserId.GetInternalId());
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();

	// If the current and previous are not the same, then release the previous
	// and send the pressed event for the current, as this is the first press

	if (CurrentPosition != PreviousPosition)
	{
		// Release the previous keys
		if (const TArray<FGamepadKeyNames::Type>* PrevKeyArray = UE::GameInput::SwitchPositionToUnrealName(PreviousPosition))
		{
			for (const FGamepadKeyNames::Type KeyName : *PrevKeyArray)
			{
				UE_LOGF(LogGameInput, Verbose, "[EvaluateSwitchState] (PlatformUserId = %d, InputDeviceId = %d) - Switch '%ls' Released", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
				Params.MessageHandler->OnControllerButtonReleased(KeyName, Params.PlatformUserId, Params.InputDeviceId, false);
			}

			RepeatTimes[static_cast<int32>(PreviousPosition)] = 0.0;
		}

		// "Press" the current key
		if (const TArray<FGamepadKeyNames::Type>* CurrentKeyArray = UE::GameInput::SwitchPositionToUnrealName(CurrentPosition))
		{
			for (const FGamepadKeyNames::Type KeyName : *CurrentKeyArray)
			{
				UE_LOGF(LogGameInput, Verbose, "[EvaluateSwitchState] (PlatformUserId = %d, InputDeviceId = %d) - Switch '%ls' Pressed", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
				Params.MessageHandler->OnControllerButtonPressed(KeyName, Params.PlatformUserId, Params.InputDeviceId, false);
			}

			RepeatTimes[static_cast<int32>(CurrentPosition)] = CurrentTime + InitialButtonRepeatDelay;
		}
	}
	// Otherwise, the states are the same. Check if we can repeat them
	else if (CurrentTime > RepeatTimes[static_cast<int32>(CurrentPosition)])
	{
		if (const TArray<FGamepadKeyNames::Type>* CurrentKeyArray = UE::GameInput::SwitchPositionToUnrealName(CurrentPosition))
		{
			for (const FGamepadKeyNames::Type KeyName : *CurrentKeyArray)
			{
				UE_LOGF(LogGameInput, Verbose, "[EvaluateSwitchState] (PlatformUserId = %d, InputDeviceId = %d) - Switch '%ls' Repeat Pressed", Params.PlatformUserId.GetInternalId(), Params.InputDeviceId.GetId(), *KeyName.ToString());
				Params.MessageHandler->OnControllerButtonPressed(KeyName, Params.PlatformUserId, Params.InputDeviceId, true);
			}
		}

		// Check for repeat key
		RepeatTimes[static_cast<int32>(CurrentPosition)] = CurrentTime + ButtonRepeatDelay;
	}

	// Keep track of the previous position
	PreviousPosition = CurrentPosition;

}

#endif	// #if GAME_INPUT_SUPPORT