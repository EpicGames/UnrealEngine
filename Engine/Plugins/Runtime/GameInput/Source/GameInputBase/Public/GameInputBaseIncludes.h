// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if GAME_INPUT_SUPPORT

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
	#include <windef.h>	// For APP_LOCAL_DEVICE_ID
#endif	// PLATFORM_WINDOWS

#include "Microsoft/COMPointer.h"

#include "GameInput.h"

#ifndef GAMEINPUT_API_VERSION
	#define GAMEINPUT_API_VERSION 0
#endif

#if GAMEINPUT_API_VERSION == 0

#elif GAMEINPUT_API_VERSION == 1

	using namespace GameInput::v1;

#elif GAMEINPUT_API_VERSION == 2

	using namespace GameInput::v2;

#elif GAMEINPUT_API_VERSION == 3

	using namespace GameInput::v3;

#endif

// Depending on the Game Input API version it will support different types of input processing.
// At the moment Game Input v1 does not support touch or raw input device processing, so we will
// compile those device processors out. "v0", which is the GDK version, does support these methods
// and will have them compiled in.

#if GAMEINPUT_API_VERSION <= 0

	#define UE_GAMEINPUT_SUPPORTS_RAW 1
	#define UE_GAMEINPUT_SUPPORTS_TOUCH 1
	#define UE_GAMEINPUT_SUPPORTS_DEVICE_STATUS 1
	#define UE_GAMEINPUT_SUPPORTS_TRIGGER_HAPTICS 0
	#define UE_GAMEINPUT_SUPPORTS_SENSORS 0
	
#elif GAMEINPUT_API_VERSION == 1 

	#define UE_GAMEINPUT_SUPPORTS_RAW 0
	#define UE_GAMEINPUT_SUPPORTS_TOUCH 0
	#define UE_GAMEINPUT_SUPPORTS_DEVICE_STATUS 0
	#define UE_GAMEINPUT_SUPPORTS_TRIGGER_HAPTICS 0
	#define UE_GAMEINPUT_SUPPORTS_SENSORS 0

#elif GAMEINPUT_API_VERSION == 2

	#define UE_GAMEINPUT_SUPPORTS_RAW 0
	#define UE_GAMEINPUT_SUPPORTS_TOUCH 0
	#define UE_GAMEINPUT_SUPPORTS_DEVICE_STATUS 0
	#define UE_GAMEINPUT_SUPPORTS_TRIGGER_HAPTICS 1
	#define UE_GAMEINPUT_SUPPORTS_SENSORS 1

#else // version 3+

	#define UE_GAMEINPUT_SUPPORTS_RAW 1
	#define UE_GAMEINPUT_SUPPORTS_TOUCH 0
	#define UE_GAMEINPUT_SUPPORTS_DEVICE_STATUS 0
	#define UE_GAMEINPUT_SUPPORTS_TRIGGER_HAPTICS 1
	#define UE_GAMEINPUT_SUPPORTS_SENSORS 1
	
#endif


THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#endif	// GAME_INPUT_SUPPORT