// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"		// For TMulticastDelegate
#include "HAL/CriticalSection.h"
#include "Modules/ModuleInterface.h"
#include "GameInputBaseIncludes.h"

#define UE_API GAMEINPUTBASE_API

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
class FGameInputHapticEndpointFactory;
#endif

class FGameInputBaseModule : public IModuleInterface
{
public:

	static UE_API FGameInputBaseModule& Get();

	/** Returns true if this module is loaded (aka available) by the FModuleManager */
	static UE_API bool IsAvailable();

	//~ Begin IModuleInterface interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

#if GAME_INPUT_SUPPORT
	/** 
	* Pointer to the static IGameInput that is created upon module startup.
	*/
	static UE_API IGameInput* GetGameInput();

	/**
	 * Delegate which is called after the creation of the IGameInput object from the game input library.
	 */
	TMulticastDelegate<void(IGameInput*)> OnGameInputCreation;
#endif

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	/**
	 * Returns the haptic endpoint factory, or null if haptic support is disabled.
	 * Created during StartupModule so it is registered before the audio device initializes.
	 */
	FGameInputHapticEndpointFactory* GetHapticFactory() const { return HapticAudioFactory.Get(); }
#endif

protected:

	UE_API void InitializeGameInputKeys();

#if PLATFORM_WINDOWS && GAME_INPUT_SUPPORT
	static UE_API FCriticalSection GameInputCreationLock;
#endif // endif PLATFORM_WINDOWS && GAME_INPUT_SUPPORT

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	void InitializeHapticAudioFactory();
	void ShutdownHapticAudioFactory();

	TUniquePtr<FGameInputHapticEndpointFactory> HapticAudioFactory;
#endif
};

#undef UE_API
