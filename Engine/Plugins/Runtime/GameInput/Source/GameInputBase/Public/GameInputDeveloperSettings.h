// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/PlatformSettings.h"
#include "GameInputBaseIncludes.h"

#include "GameInputDeveloperSettings.generated.h"

#define UE_API GAMEINPUTBASE_API

/**
 * Represents a single unique device that Microsoft's Game Input API detects.
 * Each device has a Vendor ID and  Product ID. It is possible (and common) for Game Input devices
 * to have the same VendorId's.
 *
 * This is used so that we can easily make a map of each unique device to some metadata about it's controller
 * bindings.
 *
 * If you are unsure about what the vendor or product ID's are, then you can look at the logs
 * and they will be there in the LogGameInput if you search for "VendorId" or "ProductId".
 *
 * You can also place a breakpoint on the device connection status changed to see the value there from the GameInputDeviceInfo.
 */
USTRUCT()
struct FGameInputDeviceIdentifier
{
	GENERATED_BODY()

	UE_API FGameInputDeviceIdentifier();
	UE_API FGameInputDeviceIdentifier(uint16 InVendorId, uint16 InProductId);

	/** The Vendor ID of this device */
	UPROPERTY(EditAnywhere, Config, Category = "Input Device")
	uint16 VendorId;

	/** The unique Product ID of this device */
	UPROPERTY(EditAnywhere, Config, Category = "Input Device")
	uint16 ProductId;

	GAMEINPUTBASE_API friend uint32 GetTypeHash(const FGameInputDeviceIdentifier& InId);
	UE_API bool operator==(const FGameInputDeviceIdentifier& Other) const;
	UE_API bool operator!=(const FGameInputDeviceIdentifier& Other) const;

	/**
	 * Returns a string representing this Game Input device identifier.
	 *
	 * For example:
	 *		"VendorId: XXX ProductId XXX"
	 */
	UE_API FString ToString() const;
};

/**
* UPROP friendly representation of GameInput's GameInputDeviceFamily type.
*/
UENUM(meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EGameInputDeviceFamily : uint32
{
	None 	  =  0 UMETA(Hidden),
	Virtual   =  1 << 0,
	Unknown   =  1 << 1,
	XboxOne   =  1 << 2,
	Xbox360   =  1 << 3,
	Hid       =  1 << 4,
	I8042     =  1 << 5,
	Aggregate =  1 << 6,
	All		  =  Virtual | Unknown | XboxOne | Xbox360 | Hid | I8042 | Aggregate UMETA(Hidden),
};

ENUM_CLASS_FLAGS(EGameInputDeviceFamily)

/**
* Configuration of an individual axis from a Game Input Controller device type.
* 
* These settings would be used for one individual reading from a "GetControllerAxisState" call.
* 
* @see FGameInputControllerDeviceProcessor::ProcessControllerAxisState
*/
USTRUCT()
struct FGameInputControllerAxisData
{
	GENERATED_BODY()
	
	FGameInputControllerAxisData() = default;

	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device")
	FName KeyName = NAME_None;

	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device")
	float DeadZone = (7849.0f / 32768.0f);

	/** Scalar that the input value will be multiplied by before being reported to the input system. */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device")
	float Scalar = 1.0f;

	/** 
	* Set this to true if this axis represents a positive and negative value packed into one float.
	* For example, a trigger will be a single value between 0.0 and 1.0, where 0.0 is unactuated and 1.0 is full actuated.
	* However, something like a thumb stick may pack both positive and negative values into its range, from -1.0 to +1.0 
	* where -1.0 is all the way to the left, and +1.0 is all the way to the right.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device")
	bool bIsPackedPositveAndNegative = false;
};

/***
 * Options for how to translate the raw uint8 RawInput report data to the engine.
 */
UENUM()
enum class ERawDeviceReportTranslationBehavior : uint8
{
	// Treat the value as a trigger, mapping it from 0.0 to +1.0
	TreatAsTrigger = 0,
	
	// Treat the value as an analog value, mapping it from -1.0 to +1.0
	TreatAsAnalog = 1,
	
	// Treats the raw value as a button. If the value is 0, consider it not pressed. If the value is non-zero, consider it pressed.
	TreatAsButton = 2,

	// Treat the raw value as a bitmask of different buttons. Treats each bit in the raw uint8 value as its own button.
	// 
	// If the bit is 0, then consider it not pressed. If the bit is non-zero (1) then consider it pressed. 
	TreatAsButtonBitmask = 3,

	// Treat this as two raw values, where one axis is the "lower" bits of an int16 and 
	// one is the "higher" bits of the int16. 
	TreatAsPackedAxisPair = 4,

	// Treat this raw report index as two values, reporting two different sets of buttons.
	// The lower 4 bits represent one set of buttons, and the upper 4 bits represent another set of buttons.
	TreatAsPackedAxisUint8 = 5,
};

/**
* A configuration of a single Raw Device Report index. 
* This information will be used to map the raw device report information 
* of a given index to something that the Unreal Engine can process with its
* message handler.
* 
* @see FGameInputRawDeviceProcessor::ProcessInput
*/
USTRUCT()
struct FGameInputRawDeviceReportData
{
	GENERATED_BODY()
	
	FGameInputRawDeviceReportData() = default;

	/**
	 * The name of the FKey that this raw device report is associated with when treated as an analog or trigger value
	 * 
	 * Note: This value only matters for Translation Behaviors other then the button bitmask.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta=(EditCondition="TranslationBehavior != ERawDeviceReportTranslationBehavior::TreatAsButtonBitmask && TranslationBehavior != ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8", EditConditionHides))
	FName KeyName = NAME_None;

	/**
	* If true, then this raw report data will NOT set the FInputDeviceScope.
	* 
	* This may be desirable if you have a very sensitive axis value or something which may be constantly reporting
	* that you don't necessarily want to be used to determine things like Common UI icons or other systems who read
	* device scope.
	* 
	* Default: False.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta=(EditCondition="TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsAnalog || TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsTrigger", EditConditionHides))
	bool bIgnoreAnalogInputDeviceScopeForThisRawReport = false;

	/**
	 * Options for how we should interpret the raw uint8 value from RawInput when telling the engine about it
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device")
	ERawDeviceReportTranslationBehavior TranslationBehavior = ERawDeviceReportTranslationBehavior::TreatAsTrigger;

	/**
	 * The deadzone to use when translating the value to an analog input between -1.0 and +1.0.
	 * 
	 * Normally a value between 0-10 is about where you want it for most devices.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta=(EditCondition="TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsAnalog", EditConditionHides))
	uint8 AnalogDeadzone = 5;

	/**
	* A scalar that gets applied to the raw float value for Analog or trigger values. Most of the time this would be used to simply negate values
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta=(EditCondition="TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsAnalog || TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsTrigger", EditConditionHides))
	float Scalar = 1.0f;

	/**
	 * When treating this raw input as a button we will map the key value of this map to a bit on the raw input value.
	 * I.e. a key value of "1" will look at the first bit to see if it is set. If it is, then send a controller button event
	 * for the associated FName
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta=(EditCondition="TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsButtonBitmask", EditConditionHides))
	TMap<int32, FName> ButtonBitMaskMappings;

	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta = (EditCondition = "TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisPair", EditConditionHides))
	uint8 LowerBitAxisIndex = 0;
	
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta = (EditCondition = "TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisPair", EditConditionHides))
	uint8 HigherBitAxisIndex = 0;

	/**
	* Array of FKey names to report when the lower bits of this raw value index are detected. Each element of this array will be reported once per-frame, known as an "input chord"
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta = (EditCondition = "TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8", EditConditionHides))
	TArray<FName> LowerBitKeys;

	/**
	* Array of FKey names to report when the upper bits of this raw value index are detected. Each element of this array will be reported once per-frame, known as an "input chord" 
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta = (EditCondition = "TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8", EditConditionHides))
	TArray<FName> UpperBitKeys;

	/**
	 * If true, then this packed bitmask value will be treated as an analog button.
	 * This means that it will use "OnControllerAnalog", reporting a float value, instead of "OnControllerButtonPressed" using input events. 
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input Device", meta = (EditCondition = "TranslationBehavior == ERawDeviceReportTranslationBehavior::TreatAsPackedAxisUint8", EditConditionHides))
	bool bTreatValuesAsAnalog = false;

};

/**
* Configurable data about a unique type of Game Input device. These settings
* will be used at runtime to determine the behavior of any device whose Vendor and
* Product ID match this one.
*/
USTRUCT()
struct FGameInputDeviceConfiguration
{
	GENERATED_BODY()

	UE_API FGameInputDeviceConfiguration();

	/** The unique device vendor and product ID that can be used to ID this item at runtime. */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings")
	FGameInputDeviceIdentifier DeviceIdentifier;

#if WITH_EDITORONLY_DATA
	/**
	* An editor-only description of this input device. 
	* Only used for if you want to describe your device and make the project settings easier to read 
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings", meta=(DisplayName="Description (Editor Only)"))
	FString Description = TEXT("");
#endif
	
	/** 
	* If true, then the hardware device Id specific on this configuration will be used instead of
	* whatever the Game Input SDK tells us this device is. Use this if want to get additional
	* messaging from the Input Device Subsystem when this specific device is used.
	* 
	* For example, some third party controller might be an "Xbox One" controller type, but 
	* it is really some specially manufactured hardware for your game that you would like 
	* to have access to at the Gameplay layer in UE.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings", meta = (InlineEditConditionToggle, ConfigRestartRequired=true))
	bool bOverrideHardwareDeviceIdString;

	/**
	* The name of this device that should be used for FInputDeviceScope's to determine 
	* when it has been connected or input events came from it.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings", meta=(EditCondition="bOverrideHardwareDeviceIdString", ConfigRestartRequired=true))
	FString OverriddenHardwareDeviceId;

	/** If true, this device will attempt to process Game Input Button states. */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Controller", meta = (InlineEditConditionToggle))
	bool bProcessControllerButtons;

	/** 
	* If true, this device will attempt to process Game Input Switch (aka DPad) state.
	* 
	* If the device has it's DPad being processed as buttons already, you may want to turn this off.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Controller")
	bool bProcessControllerSwitchState;
	
	/**
	 * If true, this device will attempt to process Game Input Controller Axis mappings.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Controller", meta = (InlineEditConditionToggle))
	bool bProcessControllerAxis;

	/**
	* A map of uint32 button index to an associated FName Unreal gamepad key name.
	* 
	* These key values should be (1 << [button index]) i.e, powers of two.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Controller", meta=(EditCondition="bProcessControllerButtons"))
	TMap<uint32, FName> ControllerButtonMappingData;

	/**
	* A map of uint32 button index to some data about how we should treat the raw axis value
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Controller", meta=(EditCondition="bProcessControllerAxis"))
	TMap<uint32, FGameInputControllerAxisData> ControllerAxisMappingData;

	/**
	 * If true, this device will attempt to process Game Input Raw Report mappings.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Raw Report")
	bool bProcessRawReportData;

	/** 
	* The raw report reading ID ( GameInputRawDeviceReportInfo::id ).
	* 
	* We need to know this value so that we can confirm that the reading we are receiving is for this device configuration.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Raw Report", meta=(EditCondition="bProcessRawReportData"))
	uint32 RawReportReadingId;
	
	/**
	* A map of uint32 raw report index to some data about how we should treat that raw data.
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Raw Report", meta=(EditCondition="bProcessRawReportData"))
	TMap<int32, FGameInputRawDeviceReportData> RawReportMappingData;
};

/**
 * Settings for Game Input that you may want to set per-platform.
 */
UCLASS(MinimalAPI, config = Input, DefaultConfig)
class UGameInputPlatformSettings : public UPlatformSettings
{
	GENERATED_BODY()
	
	friend class UGameInputDeveloperSettings;

protected:

	UE_API virtual void InitializePlatformDefaults() override;
	
public:

	/** Returns a pointer to Game Input settings on the current platform. */
	static UE_API UGameInputPlatformSettings* Get();

	/**
	 * If true Game Input will process controller axis, button, and switch events
	 *
	 * Default: False
	 */
	UPROPERTY(Config, EditAnywhere, Category="Processing Options|Require Configuration")
	bool bProcessController = false;

	/** 
	* If true, then we will process the GameInputKindRawDeviceReport type.
	*
	* Default: False
	*/
	UPROPERTY(Config, EditAnywhere, Category="Processing Options|Require Configuration")
	bool bProcessRawInput = false;
	
	/**
	* If true, then we will only process controller devices whose vendor/product ID is in the 
	* DeviceConfigurations array. This applies to both Controller and Raw input kinds. 
	* 
	* Default: True
	*/
	UPROPERTY(Config, EditAnywhere, Category="Processing Options|Require Configuration", meta=(EditCondition="bProcessController || bProcessRawInput"))
	bool bSpecialDevicesRequireExplicitDeviceConfiguration = true;

	/** 
	* If true, then we will process the GameInputKindGamepad type.
	* 
	* Note: If you are using Game Input on Windows where there are other Input Device module plugins (XInput, WinDualShock, etc) 
	* you should disable those to use this. Otherwise, there will be "duplicate" gamepad input events.
	* 
	* Default: True
	*/
	UPROPERTY(Config, EditAnywhere, Category="Processing Options")
	bool bProcessGamepad = true;

	/** 
	* If true, then we will process the GameInputKindKeyboard type.
	* 
	* Note: You likely do not want this on for Windows targets, as those input events are already processed via the 
	* WindowsApplication. This should really only be used for console targets. This is currently disabled on Windows.
	* 
	* Default: True
	*/
	UPROPERTY(Config, EditAnywhere, Category="Processing Options")
	bool bProcessKeyboard = true;

	/** 
	* If true, then we will process the GameInputKindMouse type.
	* 
	* Note: You likely do not want this on for Windows targets, as those input events are already processed via the 
	* WindowsApplication. This should really only be used for console targets. This is currently disabled on Windows.
	* 
	* Default: True
	*/
	UPROPERTY(Config, EditAnywhere, Category="Processing Options")
	bool bProcessMouse = true;

	/**
	 * If true, then we will process the GameInputKindSensors type such as Gyrometer, Orientation, and Accelerometer.
	 *
	 * Experimental.
	 */
	UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
	UPROPERTY(Config, EditAnywhere, Category="Processing Options")
	bool bProcessSensors = false;

	/**
	 * If true, then we will process the GameInputKindRacingWheel type.
	 * 
	 * Racing Wheels often times need to be used in conjunction with the "Controller" or "Gamepad" processors as well
	 * in order to handle any "normal" buttons on them. For example, you may have a racing wheel with some ABXY buttons on it
	 * that you want to process as well.
	 * 
	 * Note: This is experimental!
	 * 
	 * Default: False
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Processing Options|Require Configuration", meta=(DisplayName="Process Racing Wheel (Experimental)"))
	bool bProcessRacingWheel = false;

	/** The default racing wheel deadzone */
	static constexpr float DefaultRacingWheelDeadzone = (7849.0f / 32768.0f);

	/**
	* The deadzone that should be applied when processing Racing Wheel analog values.
	* 
	* @see FGameInputRacingWheelProcessor::ProcessWheelAnalogState
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Racing Wheel", meta=(EditCondition="bProcessRacingWheel"))
	float RacingWheelDeadzone = DefaultRacingWheelDeadzone;

	/**
	* If true, then we will process the GameInputKindArcadeStick type.
	* 
	* Note: This is experimental!
	* 
	* Default: False
	*/
	UPROPERTY(EditAnywhere, Config, Category = "Processing Options", meta=(DisplayName="Process Arcade Sticks (Experimental)"))
	bool bProcessArcadeStick = false;

	/**
	 * If true, then we will process the GameInputKindFlightStick type.
	 *
	 * Note: This is experimental!
	 *
	 * Default: False
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Processing Options", meta=(DisplayName="Process Flight Sticks (Experimental)"))
	bool bProcessFlightStick = false;

	/** The default value of any flight stick deadzone values. */
	static constexpr float DefaultFlightStickDeadzone = (7849.0f / 32768.0f);

	/** Deadzone which should be used for Flight Stick Pitch inputs */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Flight Stick", meta=(EditCondition="bProcessFlightStick"))
	float FlightStickPitchDeadzone = DefaultFlightStickDeadzone;

	/** Deadzone which should be used for Flight Stick Roll inputs */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Flight Stick", meta=(EditCondition="bProcessFlightStick"))
	float FlightStickRollDeadzone = DefaultFlightStickDeadzone;

	/** Deadzone which should be used for Flight Stick Throttle inputs */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Flight Stick", meta=(EditCondition="bProcessFlightStick"))
	float FlightStickThrottleDeadzone = DefaultFlightStickDeadzone;

	/** Deadzone which should be used for Flight Stick Yaw inputs */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings|Flight Stick", meta=(EditCondition="bProcessFlightStick"))
	float FlightStickYawDeadzone = DefaultFlightStickDeadzone;
};

/**
* Settings related to the Game Input device interface. 
* 
* These will allow you to enable and disable specific types of input devices within Game Input
* as well as configure key mappings for generic controller types based on their unique vendor/product ID's.
*/
UCLASS(MinimalAPI, config = Input, DefaultConfig, meta = (DisplayName = "Game Input Plugin Settings"))
class UGameInputDeveloperSettings : public UObject
{
	GENERATED_BODY()
public:

#if GAME_INPUT_SUPPORT
	/**
	* Finds unique device configuration data for the given Game Input Device. This is based
	* on the vendor/product ID of the game input device
	* 
	* @return Device Configuration for the given Game Input device. Null if it is not configured.
	*/
	[[nodiscard]] UE_API const FGameInputDeviceConfiguration* FindDeviceConfiguration(const GameInputDeviceInfo* const Info) const;

	/**
	 * Returns a pointer to the device configuration that may exist for the given hardware ID
	 */
	UE_API const FGameInputDeviceConfiguration* FindDeviceConfiguration(const FGameInputDeviceIdentifier& HardwareID) const;

	/**
	* Returns true if the gien device should be completely ignored by GameInput processing.
	* If a device is ignored, then it will not even fire connection event delegates.
	*/
	bool ShouldIgnoreDevice(const GameInputDeviceInfo* const Info) const; 
	
#endif	 // GAME_INPUT_SUPPORT

	UE_API UGameInputDeveloperSettings(const FObjectInitializer& Initializer);

protected:
	
	/**
	 * Array of devices that you want to specify the behavior for. These could be some
	 * special input devices for your game like racing wheels, instruments, or other "special"
	 * peripherals that require some kind of special readings from Game Input.
	 *
	 * This is typically for GameInputKindController or GameInputKindRawDeviceReport.
	 *
	 * If bSpecialDevicesRequireExplicitDeviceConfiguration is true, only Controller/Raw device kinds
	 * that are within this array will be allowed to be processed.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Game Input")
	TArray<FGameInputDeviceConfiguration> DeviceConfigurations;

	/**
	 * Settings for Game input that you may want to set per-platform.
	 *
	 * This is useful to have per-platform because some platforms might already handle the processing
	 * of some input devices, so you might want to disable them here.
	 */
	UPROPERTY(EditAnywhere, Category = "Game Input|Platform Options")
	FPerPlatformSettings PlatformSpecificSettings;

	/**
	 * A list of devices for game input to completely ignore. If a device is connected that is in this list,
	 * no processors will ever be created for it at all.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings")
	TArray<FGameInputDeviceIdentifier> DevicesToIgnore;

	/**
	 * A bitmask of which GameInput device families you want to support. 
	 * If a family is not enabled as a part of this list, it will be ignored.
	 * 
	 * Default: All families are supported.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Device Settings", meta = (Bitmask, BitmaskEnum = "/Script/GameInputBase.EGameInputDeviceFamily"))
	uint32 SupportedDeviceFamilies;

public:
	/**
	* If true, when you have multiple input devices mapped to a single FPlatformUserId,
	* we will only process events from one of them each frame. This makes it impossible for
	* analog input values to "over accumulate" and report values higher than +- 1.0
	*
	* One example of this issue would be if a user has mapped two game pads to the same platform user
	* and them presses the right analog stick all the way to the right on both of them at the same time. When this setting
	* is on, we will only read the value from one of the sticks, resulting in an input value of +1.0. 
	* With this setting off (which is not recommended) then you would receive an input value of +2.0. 
	* This could result in having faster rotations around in your game or other non-desirable behaviors. 
	*
	* WARNING: Turning this off may result in Slate receiving input events greater than 1.0, which can cause 
	* undefined behavior in your if it makes the assumption of input being between -1.0 and +1.0.
	* 
	* Default: True
	*/
	UPROPERTY(Config, EditAnywhere, Category="Game Input")
	bool bDoNotProcessDuplicateCapabilitiesForSingleUser;

	/**
	 * @return True if this build target supports haptics (UE_GAME_INPUT_SUPPORTS_HAPTICS) and the project setting
	 * for enabling haptic support is true.
	 */
	bool IsHapticSupportEnabled() const;
	
protected:
	
	/**
	 * If true, then Game Input will listen for the GameInputDeviceHapticInfoReady device status on platforms
	 * which UE_GAME_INPUT_SUPPORTS_HAPTICS is true.
	 *
	 * Haptic sensor support is like a "fancy" force feedback. On supported devices, an audio endpoint is created.
	 * You can use audio files to drive sound and vibration on the devices.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Experimental|Game Input")
	bool bEnableHapticSensorSupport;

#if WITH_EDITOR
public:

	/** A delegate that is called on PostEditChangeProperty with the name of the property that has changed. */
	TMulticastDelegate<void()> OnInputSettingChanged;
	
	/**
	 * Returns all known uint32 options for controller button configurations.
	 * this is useful for making it easier to select in the editor instead of having to manually type everything in
	 * on a UPROPERTY
	 * i.e.
	 *
	 * meta=(GetOptions="GameInput.GameInputDeveloperSettings.GetControllerButtonMappingDataKeyOptions")
	 */
	UFUNCTION()
	static UE_API const TArray<uint32>& GetControllerButtonMappingDataKeyOptions();

protected:
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif	// WITH_EDITOR
};

#undef UE_API
