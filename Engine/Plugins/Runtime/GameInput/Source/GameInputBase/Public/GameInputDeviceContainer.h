// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

#define UE_API GAMEINPUTBASE_API

#if GAME_INPUT_SUPPORT

/**
* Game Input Device Containers hold any state about a single unique GameInput device.
* Upon creation, the will create any "Device Processors" that are required based 
* on the IGameInputDevice's supported GameInputKind. They are ticked each frame by the 
* owning IGameInputDeviceInterface who created them. 
* 
* This class is what will actually poll the GameInput SDK for new readings and keep track
* of it's state across frames. 
*/
class FGameInputDeviceContainer
{
public:
	UE_API FGameInputDeviceContainer(
		const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler,
		IGameInputDevice* InDevice,
		GameInputKind InAllowedGameInputKinds,
		FPlatformUserId InUserId = PLATFORMUSERID_NONE, 
		FInputDeviceId InDeviceId = INPUTDEVICEID_NONE);

	virtual ~FGameInputDeviceContainer() = default;

	/** Create any device processors necessary for this Game Input device to process input and send input events */
	UE_API void InitalizeDeviceProcessors();

	/**
	 * Clears the state of any device processors and destroyies them. Empties the processors array.
	 * Sets the allowed input kind to the new given input kinds
	 */
	UE_API void RecreateDeviceProcessors(const GameInputKind InAllowedGameInputKinds);

	/** Set the message handler that this Device Container should send events through. */
	UE_API void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	
	/**
	 * Process any input events from the given reading and send events to a message handler
	 * 
	 * @param 	GameInput								Pointer to the game input instance
	 * 
	 * @param 	CurrentSupportedKind					Currently supported game input reading kinds that we are allowed to process. This will be used to get the 
	 *													current readings from GameInput with GameInput->GetNextReading
	 * 
	 * @param 	ProcessedKindsForPlatformUserThisFrame	A bitmask of any game input kinds that have already been processed by the platform user this frame.
	 *													If the UGameInputDeveloperSettings::bDoNotProcessDuplicateCapabilitiesForSingleUser flag is true,
	 *													then will skip processors whose input kind is set in this bitmask.
	 * 
	 * Returns a bitmask of what GameInputKind's were processed this frame.
	 */
	UE_API const GameInputKind ProcessInput(IGameInput* GameInput, const GameInputKind CurrentSupportedKind, const GameInputKind ProcessedKindsForPlatformUserThisFrame);

	/** Reset any input state that is necessary. This would be called when the application is no longer constrained for example */
	UE_API void ClearInputState(IGameInput* GameInput);

	/**
	* Returns a pointer to the IGameInputDevice that this device is associated with
	* 
	* If this is null, then this is a container for an input device which has been disconnected.
	*/
	UE_API IGameInputDevice* GetGameInputDevice() const;

	/**
	* Update the IGameInputDevice pointer that this container should use. 
	*
	* This IGameInputDevice should have the same Local App ID as this container already, otherwise there will be an ensure
	* 
	* @param InDevice	The new IGameInputDevice that this container should use. 
	*/
	UE_API void SetGameInputDevice(IGameInputDevice* InDevice);

	/** Returns the unique local device ID of this IGameInputDevice from the GameInput API. */
	UE_API APP_LOCAL_DEVICE_ID GetGameInputDeviceId() const;

	UE_API void SetPlatformUserId(const FPlatformUserId InUserId);
	UE_API FPlatformUserId GetPlatformUserId() const;

	UE_API void SetInputDeviceId(const FInputDeviceId InDeviceId);
	UE_API FInputDeviceId GetDeviceId() const;

	UE_API uint64 GetLastReadingTimestamp() const;

	/** 
	* Returns the number of processors that this device container currently has.
	*
	* Note: If this is zero, then this device can't possibly fire any input events.
	*/
	UE_API const int32 GetNumberOfProcessors() const;

protected:

	/** Create any device processors necessary for this Game Input device to process input and send input events */
	UE_API virtual void InitalizeDeviceProcessors_Impl();
	
	/** Message handler that we can use to tell the engine about input events */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	/** The owning Game Input device that this device is associated with */
	IGameInputDevice* Device;

	/** The kinds of Input that are allowed to be created for this device. */
	GameInputKind AllowedGameInputKinds;

	/** The owning Platform User ID that this device is associated with */
	FPlatformUserId UserId;

	/** The Input Device ID that this Game Input Device is associated with */
	FInputDeviceId AssignedDeviceId;

	/** 
	* The unique ID that is associated with this device from the GameInput API.
	* This will be set only once upon construction of this container.
	* 
	* @see IGameInputDeviceInterface::GetDeviceData
	* @see https://learn.microsoft.com/en-us/gaming/gdk/_content/gc/reference/system/xuser/structs/app_local_device_id
	*/
	APP_LOCAL_DEVICE_ID LocalDeviceId;

	/** Input Device processors to handle input for any specific types required (gamepad, racing wheel, KBM, etc) */
	TArray<TSharedPtr<IGameInputDeviceProcessor>> Processors;

	/** The last reading that was processed by this device */
	TComPtr<IGameInputReading> LastReading;

	/** Timestamp that can be used to check if we should ignore a reading from the past or not */
	uint64 IgnoreReadingTimestamp;

	/** Timestamp of the last reading processed by this device */
	uint64 LastReadingTimestamp;
};

#endif	// GAME_INPUT_SUPPORT

#undef UE_API
