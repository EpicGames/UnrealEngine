// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.8, "Use the specific device processor header file you need instead of this.")

// These are the new device processor header files you can use instead.
#include "Processors/GameInputDeviceProcessor_ArcadeStick.h"
#include "Processors/GameInputDeviceProcessor_Controller.h"
#include "Processors/GameInputDeviceProcessor_FlightStick.h"
#include "Processors/GameInputDeviceProcessor_Gamepad.h"
#include "Processors/GameInputDeviceProcessor_KeyboardMouse.h"
#include "Processors/GameInputDeviceProcessor_RacingWheel.h"
#include "Processors/GameInputDeviceProcessor_Raw.h"
#include "Processors/GameInputDeviceProcessor_Sensor.h"
#include "Processors/GameInputDeviceProcessor_Touch.h"
#include "Processors/IGameInputDeviceProcessor.h"