// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAudioEndpoint.h"

#include "GameInputHapticEndpointSettings.generated.h"

/**
 * Thread-safe proxy for UGameInputHapticEndpointSettings, used on the audio thread.
 *
 * Currently empty — the factory binds endpoints to controllers based on connection
 * order at the audio-device level, not via per-submix settings. Kept as a distinct
 * type so additional per-endpoint knobs (e.g. attenuation, channel remap) can be
 * added without changing the factory contract.
 */
struct FGameInputHapticEndpointSettingsProxy : public IAudioEndpointSettingsProxy
{
};

/**
 * Settings asset for the "Vibration Output" audio endpoint.
 * Assign this to a Sound Submix Endpoint asset to route audio to a GameInput
 * haptic-capable controller's haptic motors.
 *
 * The factory currently routes haptic audio to the first connected haptic controller.
 * Multi-controller routing is not yet supported — see FGameInputHapticEndpointFactory
 * if/when per-controller targeting is required.
 */
UCLASS()
class UGameInputHapticEndpointSettings : public UAudioEndpointSettingsBase
{
	GENERATED_BODY()

public:

	virtual TUniquePtr<IAudioEndpointSettingsProxy> GetProxy() const override
	{
		return TUniquePtr<IAudioEndpointSettingsProxy>(new FGameInputHapticEndpointSettingsProxy());
	}
};
