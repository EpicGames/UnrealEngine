// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"

#define UE_API GAMEINPUTBASE_API

/**
 * Key types that are unique to the Game Input plugin
 * These keys are added on module startup of the Game Input module.
 *
 * HEY, LISTEN!
 * If you add or otherwise modify these keys, make sure the change is reflected in FGameInputBaseModule::InitializeGameInputKeys.
 */
struct FGameInputKeys
{
	//
	// Racing Wheel
	//

	// Analog types
	static UE_API const FKey RacingWheel_Brake;
	static UE_API const FKey RacingWheel_Clutch;
	static UE_API const FKey RacingWheel_Handbrake;
	static UE_API const FKey RacingWheel_Throttle;
	static UE_API const FKey RacingWheel_Wheel;
	static UE_API const FKey RacingWheel_PatternShifterGear;

	// Button types
	static UE_API const FKey RacingWheel_None;
	static UE_API const FKey RacingWheel_Menu;
	static UE_API const FKey RacingWheel_View;
	static UE_API const FKey RacingWheel_PreviousGear;
	static UE_API const FKey RacingWheel_NextGear;

	//
	// Flight Stick
	//

	// Analog types
	static UE_API const FKey FlightStick_Roll;
	static UE_API const FKey FlightStick_Pitch;
	static UE_API const FKey FlightStick_Yaw;
	static UE_API const FKey FlightStick_Throttle;

	// Button types
	static UE_API const FKey FlightStick_None;
	static UE_API const FKey FlightStick_Menu;
	static UE_API const FKey FlightStick_View;
	static UE_API const FKey FlightStick_FirePrimary;
	static UE_API const FKey FlightStick_FireSecondary;

	//
	// Arcade Stick
	//

	// Button Types
	static UE_API const FKey ArcadeStick_Action1;
	static UE_API const FKey ArcadeStick_Action2;
	static UE_API const FKey ArcadeStick_Action3;
	static UE_API const FKey ArcadeStick_Action4;
	static UE_API const FKey ArcadeStick_Action5;
	static UE_API const FKey ArcadeStick_Action6;
	static UE_API const FKey ArcadeStick_Special1;
	static UE_API const FKey ArcadeStick_Special2;

	//
	// Sensor Types
	//
	UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
	static UE_API const FKey Sensor_HeadingInDegreesFromMagneticNorth;
};

#undef UE_API
