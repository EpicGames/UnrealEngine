// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
class FGameInputHapticEndpointFactory;
#endif

#if GAME_INPUT_SUPPORT

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GameInputBaseIncludes.h"

namespace UE::GameInput
{
	/**
	 * @return True if there is a valid version of Game Input currently available
	 */
	GAMEINPUTBASE_API bool HasValidVersionOfGameInput();
	
	/**
	* Returns a string which represents the version of Game Input that is installed on this machine.
	* "Unknown" will be used if no valid version is found. 
	*/
	GAMEINPUTBASE_API FString GetInstalledGameInputVersionInfoString();
	
	/**
	* Returns a string which represents the min required version of Game Input needed to run the engine. 
	* "Unknown" will be used if no valid version is found.
	*/
	GAMEINPUTBASE_API FString GetMinVersionOfGameInputRedistString();
	
	GAMEINPUTBASE_API FString LexToString(IGameInputDevice* Device);
	
	GAMEINPUTBASE_API FString LexToString(GameInputDeviceStatus DeviceStatus);

	GAMEINPUTBASE_API FString LexToString(GameInputKind InputKind);

	GAMEINPUTBASE_API FString LexToString(GameInputSwitchPosition SwitchPos);

#if UE_GAMEINPUT_SUPPORTS_SENSORS
	GAMEINPUTBASE_API FString LexToString(GameInputSensorsKind SensorKind);

	GAMEINPUTBASE_API FString LexToString(GameInputSensorAccuracy SensorKind);
#endif	// #if UE_GAMEINPUT_SUPPORTS_SENSORS

	GAMEINPUTBASE_API FString LexToString(const APP_LOCAL_DEVICE_ID& DeviceId);

	GAMEINPUTBASE_API const TCHAR* GetMouseButtonName(EMouseButtons::Type MouseButton);

	GAMEINPUTBASE_API EInputDeviceConnectionState DeviceStateToConnectionState(GameInputDeviceStatus InCurrentStatus, GameInputDeviceStatus InPreviousStatus);

	/**
	 * Populates the OutKeyName with the name of the Unreal Engine FKey that should be used for the given button mask, based on the
	 * UE button map that should be used.
	 *
	 * @return true if a key name was found, false otherwise.
	 */
	GAMEINPUTBASE_API bool GameInputButtonToUnrealName(const TMap<uint32, FGamepadKeyNames::Type>& UEButtonMap, uint32 ButtonMask, OUT FGamepadKeyNames::Type& OutKeyName);

	/** A map of game input switch position enums to Unreal Engine FKey names */
	GAMEINPUTBASE_API const TMap<GameInputSwitchPosition, TArray<FGamepadKeyNames::Type>>& GetSwitchButtonMap();

	/**
	* Gets an array of FKeys that are associated with the given switch position.
	* The array can have multiple keys if the GameInput switch state is "Up/Down + Left/Right"
	*/
	GAMEINPUTBASE_API const TArray<FGamepadKeyNames::Type>* SwitchPositionToUnrealName(const GameInputSwitchPosition ButtonMask);

	GAMEINPUTBASE_API const GameInputDeviceInfo* GetDeviceInfo(IGameInputDevice* Device);

	GAMEINPUTBASE_API APP_LOCAL_DEVICE_ID GetDeviceAppLocalDeviceId(IGameInputDevice* Device);

#if UE_GAME_INPUT_SUPPORTS_HAPTICS
	/** Returns the haptic endpoint factory, or null if haptic support is disabled or GameInput is not the preferred input device. */
	GAMEINPUTBASE_API ::FGameInputHapticEndpointFactory* GetHapticEndpointFactory();
#endif

	/** The FName which should be used to construct FInputDeviceScope's for game input devices. */
	GAMEINPUTBASE_API FName GetDeviceScopeClassName();

	/** 
	* Returns the string hardware device identifier for the given input event params that can be used
	* to create an FInputDeviceScope
	*/
	GAMEINPUTBASE_API FStringView GetHardwareDeviceIdentifierName(IGameInputDevice* Device);
	
	// Button flags assigned by GameInput
	enum { GamingInputButtonMask = 0x3FFF };

	// Allocate flags for emulated/extended digital button functionality
	enum EGameInputAuxButton
	{
		GamepadButtonAux_LeftTrigger = (1 << 22),
		GamepadButtonAux_RightTrigger = (1 << 23),
		GamepadButtonAux_LeftStickUp = (1 << 24),
		GamepadButtonAux_LeftStickDown = (1 << 25),
		GamepadButtonAux_LeftStickLeft = (1 << 26),
		GamepadButtonAux_LeftStickRight = (1 << 27),
		GamepadButtonAux_RightStickUp = (1 << 28),
		GamepadButtonAux_RightStickDown = (1 << 29),
		GamepadButtonAux_RightStickLeft = (1 << 30),
		GamepadButtonAux_RightStickRight = (1 << 31),
	};

	// TODO_BH: Move these into a nice settings object instead of being hard coded
	// Values taken from XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE, XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE, XINPUT_GAMEPAD_TRIGGER_THRESHOLD used by XInput.
	static constexpr float GamepadLeftStickDeadzone = (7849.0f / 32768.0f);
	static constexpr float GamepadRightStickDeadzone = (8689.0f / 32768.0f);
	static constexpr float GamepadTriggerDeadzone = (30.0f / 255.0f);
	static constexpr float SensorMagneticNorthDeadzone = (30.0f / 255.0f);

	static constexpr float InitialRepeatDelay = (0.2f);
	static constexpr float SubsequentRepeatDelay = (0.1f);
}

#endif	// GAME_INPUT_SUPPORT