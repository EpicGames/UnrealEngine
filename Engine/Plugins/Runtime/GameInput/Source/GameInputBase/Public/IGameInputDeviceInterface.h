// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDevice.h"
#include "Misc/CoreMiscDefines.h"	// For FInputDeviceId/FPlatformUserId
#include "GameInputBaseIncludes.h"
#include "GameInputDeviceContainer.h"

/**
* IInputDevice interface for the Game Input library. This is a base class
* that is intended to be implemented on each platform that supports it (Xbox and PC).
* 
* This stores all state about every unique Human Interface Device that the GameInput library
* detects. 
* 
* Any implementors of this interface should be Created in an IInputDevice module's CreateInputDevice function
* (IInputDeviceModule::CreateInputDevice), and have it's Initialize function called
*/
class IGameInputDeviceInterface : public IInputDevice
{
public:
	GAMEINPUTBASE_API explicit IGameInputDeviceInterface(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, struct IGameInput* InGameInput);
	GAMEINPUTBASE_API virtual ~IGameInputDeviceInterface() override;
	
	/** 
	* Init function that should be called post-construction of this object. 
	* This is required for sending controller events and will ensure if not called.
	*/
	GAMEINPUTBASE_API void Initialize();

protected:

	//~ Begin IInputDevice interface
	// GameInput polling is designed for multi-threaded access per Microsoft docs.
	virtual EInputDeviceThreadAffinity GetThreadAffinity() const override { return EInputDeviceThreadAffinity::InputThreadSafe; }
	GAMEINPUTBASE_API virtual void Tick(float DeltaTime) override;
	GAMEINPUTBASE_API virtual void SendControllerEvents() override;
	GAMEINPUTBASE_API virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	GAMEINPUTBASE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	GAMEINPUTBASE_API virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	GAMEINPUTBASE_API virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values) override;
	GAMEINPUTBASE_API virtual bool IsGamepadAttached() const override;
	//~End IInputDevice interface

#if GAME_INPUT_SUPPORT

	/** Returns a mask of the currently supported Game Input Kind that we want to get the reading of. */
	GAMEINPUTBASE_API virtual GameInputKind GetCurrentGameInputKindSupport() const;

	/** Binds a listener for device status changes from game input to this class */
	GAMEINPUTBASE_API virtual bool BindDeviceStatusCallbacks();
	
	/**
	 * A callback for when an input device connection change has been detected.
	 */
	GAMEINPUTBASE_API virtual void OnDeviceConnectionChanged(GameInputCallbackToken CallbackToken, IGameInputDevice* Device, uint64 Timestamp, GameInputDeviceStatus CurrentStatus, GameInputDeviceStatus PreviousStatus);

	/**
	 * Returns true if the given IGameInputDevice is one which is allowed to be processed based on the Game Input Developer Settings.
	 * 
	 * This may return false for devices which have specific PID/VID's which this project does not wish GameInput to process,
	 * or if the device is within a specific family that we do not want to process.
	 */
	GAMEINPUTBASE_API virtual bool IsSupportedDevice(IGameInputDevice* Device) const;

	/**
	 * Mark this device as disconnected and invalidate the tracked game input device. A disconnected device will stay in the
	 * GameInputSourceDevices with a null Device pointer.
	 */
	virtual void HandleDeviceDisconnected(IGameInputDevice* Device, uint64 Timestamp) = 0;

	/**
	 * Map this new input device to a platform user and initialize the known FGameInputDeviceInfo.
	 * If this is a new connection, this will request a new FInputDeviceId from the platform input device mapper
	 * and add it to the GameInputSourceDevices map.
	 */
	virtual void HandleDeviceConnected(IGameInputDevice* Device, uint64 Timestamp) = 0;

private:
	
	/**
	 * Handle haptics being notified as ready from the given device.
	 * 
	 * This should only be called internally from this device interface in response to
	 * a callback from the GameInput api
	 */
	void HandleHapticsReady(IGameInputDevice* Device, uint64 Timestamp);

	GAMEINPUTBASE_API void TeardownHapticAudio(IGameInputDevice* Device);
	
protected:
	
	/**
	 * Any platform specific haptic setup can be done here in an override.
	 * The base implementation sets up WASAPI-backed audio endpoints for haptic audio.
	 *
	 * Note: You likely want to wrap the implementation of this function in #if UE_GAME_INPUT_SUPPORTS_HAPTICS.
	 */
	GAMEINPUTBASE_API virtual void HandleHapticsReady_Impl(IGameInputDevice* Device, uint64 Timestamp, struct GameInputHapticInfo& HapticInfo);

	/**
	 * Tears down the WASAPI haptic audio device associated with the given GameInput device on disconnect.
	 * Only has effect when UE_GAME_INPUT_SUPPORTS_HAPTICS is true.
	 */
	GAMEINPUTBASE_API virtual void TeardownHapticAudio_Impl(IGameInputDevice* Device);

	/** Process any deferred device connection changes that may have occurred. This should only be called on the game thread. */
	GAMEINPUTBASE_API void ProcessDeferredDeviceConnectionChanges();

	void CoalesceDeferredDeviceConnectionChanges();
	
	/** Callbacks for when the app is constrained/unconstrained (i.e. you Alt + Tab out of the window, or open the home menu on a console platform) */
	GAMEINPUTBASE_API void OnAppConstrained();
	GAMEINPUTBASE_API void OnAppUnconstrained();
	
	/** Called on the first update after the was un-constrained. */
	GAMEINPUTBASE_API void DetermineStateAfterFirstUnconstrainedUpdate();

	/** Returns a matching FGameInputDeviceContainer if one exists for the given GameInput device. Nullptr if none exist. */
	GAMEINPUTBASE_API FGameInputDeviceContainer* GetDeviceData(IGameInputDevice* InDevice);

	/**
	* Returns a pointer to the Device info associated with the given IGameInputDevice.
	*
	* If one doesn't exist, then create a new one and return that.
	*/
	GAMEINPUTBASE_API FGameInputDeviceContainer* GetOrCreateDeviceData(IGameInputDevice* InDevice);

	/**
	* Creates a new FGameInputDeviceContainer* and stores it in DeviceData. It is up to each implementing 
	* device interface to how that device is mapped to platform users and initialized.
	*/
	virtual FGameInputDeviceContainer* CreateDeviceData(IGameInputDevice* InDevice) = 0;

	/**
	* This will update the CurrentlyConnectedDeviceTypes bitmask based on the current set of DeviceData
	*/
	GAMEINPUTBASE_API virtual void EnumerateCurrentlyConnectedDeviceTypes();
	
	/** Message handler that we can use to tell the engine about input events */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;
	
	/** Pointer to the Game Input interface object */
	TComPtr<IGameInput> GameInput;

	// CS for OnDeviceConnectionChanged called from background system thread and PollGameDeviceState called from Main thread
	FCriticalSection DeviceInfoCS;

	/** Token for when a device connection change happens */
	GameInputCallbackToken ConnectionChangeCallbackToken;
	
	/** An enumeration of all the currently registered Game input Devices. */
	GameInputKind CurrentlyConnectedDeviceTypes;

	/** Set to true if the IGameInputDeviceInterface::Initialize function was called. This is required for sending controller events. */
	bool bWasinitialized;

	/** True if the app is currently constrained this tick */
	bool bIsAppCurrentlyConstrained;

	/** True if the app was constrained last tick */
	bool bWasAppConstrainedLastTick;

	// Array of Input Device data for every connected Game Input Device.
	TArray<TSharedPtr<FGameInputDeviceContainer>> DeviceData;

	// A mapping of a platform user to the most recently used Input Device for that user
	TMap<FPlatformUserId, TSharedPtr<FGameInputDeviceContainer>> PlatformUserIdToMostRecentDeviceContainer;	

	/** Stores some data about when device connection changes happen to be processed later */
	struct FDeferredDeviceConnectionChanges
	{
		FDeferredDeviceConnectionChanges() = default;

		/** The GameInput device that has changed */
		IGameInputDevice* Device = nullptr;
		
		/** Timestamp of when the connection state change happened */
		uint64 Timestamp = 0;
		
		/** The evaluated status of the game input device */
		EInputDeviceConnectionState Status = EInputDeviceConnectionState::Invalid;
		
		/** The current GameInput status of the device */
		GameInputDeviceStatus CurrentStatus = GameInputDeviceStatus::GameInputDeviceNoStatus;

		/** The previous GameInput status of the device */
		GameInputDeviceStatus PreviousStatus = GameInputDeviceStatus::GameInputDeviceNoStatus;
	};

	/** Array of device connection changed events that will be processed in ProcessDeferredDeviceConnectionChanges. */
	TArray<FDeferredDeviceConnectionChanges> DeferredDeviceConnectionChanges;

#endif	// #if GAME_INPUT_SUPPORT
};
