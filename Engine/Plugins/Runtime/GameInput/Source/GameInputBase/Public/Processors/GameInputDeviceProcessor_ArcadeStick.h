// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

/**
* Processor for the GameInputKindArcadeStick type.
* 
* These are typically accessories to "fighting" style games, or other arcade style games.
*/
class FGameInputArcadeStickProcessor : public IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API FGameInputArcadeStickProcessor();

protected:

	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	/** The previously processed arcade stick state. */
	GameInputArcadeStickState PreviousState;

	/**
	* Array of repeat times to calculate if a button has been held long enough
	* to receive an IE_REPEAT event.
	*/
	static constexpr uint32 MaxSupportedButtons = 16;
	double RepeatTime[MaxSupportedButtons];
};

#endif	// GAME_INPUT_SUPPORT