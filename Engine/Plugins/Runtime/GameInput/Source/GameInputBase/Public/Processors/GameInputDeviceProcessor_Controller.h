// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

struct FGameInputDeviceConfiguration;

/**
* Processor for Controller Input Types
* 
* Controllers are very similar to Gamepad types, and often can just be 
* a third party controller type that is not natively supported by the "GamepadInputKind".
* This can include things like instruments, other platform controller types, or other 
* third party input devices.
*/
class FGameInputControllerDeviceProcessor : public IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API FGameInputControllerDeviceProcessor();

protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	/** Processes Controller buttons (often face buttons, like ABXY or Circle/Triangle/Square/X) */
	GAMEINPUTBASE_API bool ProcessControllerButtonState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* Config, IGameInputReading* InputReading);

	/**
	* Process any axis readings on this controller.
	* This will only process axises that are configured in the given Config.
	*/
	GAMEINPUTBASE_API bool ProcessControllerAxisState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* Config, IGameInputReading* InputReading);

	/**
	* Process the controller switch (also known as the DPad) state
	*/
	GAMEINPUTBASE_API bool ProcessControllerSwitchState(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* Config, IGameInputReading* InputReading);

	/** A map of GameInputLabel types to Unreal Engine FKey names. */
	static GAMEINPUTBASE_API const TMap<GameInputLabel, FGamepadKeyNames::Type>& GetGameInputButtonLabelToUnrealName();

	/** Bitmask of any held buttons from the last frame */
	uint32 LastButtonHeldMask = 0;

	/** Max number of buttons this processor supports */
	enum { MaxSupportedButtons = 32 };

	/** Timings of when a key press should be considered a "repeat" key */
	double RepeatTime[MaxSupportedButtons];

	/** Repeat times for when switches are pressed (aka the DPad) */
	TArray<double> SwitchRepeatTimes;

	/** Previous frame's axis values */
	TArray<float> PreviousControllerAxisValues;

	/** Array of switch positions */
	TArray<GameInputSwitchPosition> PreviousSwitchPositions;

	/**
	* Keeps track of how many times this gamepad has been processed this frame.
	* Every successful processing of button input in ProcessInput will increment this value.
	* This will get reset at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;
};


#endif	// GAME_INPUT_SUPPORT