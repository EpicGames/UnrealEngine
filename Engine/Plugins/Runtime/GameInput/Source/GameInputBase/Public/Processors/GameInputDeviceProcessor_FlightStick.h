// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

/**
 *  Processor for the GameInputKindFlightStick type.
 */
class FGameInputFlightStickProcessor : public IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API FGameInputFlightStickProcessor();

protected:

	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	/**
	 * Process Flight stick specific buttons.
	 */
	GAMEINPUTBASE_API bool ProcessFlightStickButtons(const FGameInputEventParams& Params, GameInputFlightStickState& State);

	/**
	 * Process the analog values (yaw, pitch, roll, and throttle) of the flight stick
	 */
	GAMEINPUTBASE_API bool ProcessFlightStickAnalog(const FGameInputEventParams& Params, GameInputFlightStickState& State);
	
	/**
	* Keeps track of how many times this gamepad has been processed this frame.
	* Every successful processing of button input in ProcessInput will increment this value.
	* This will get reset at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;
	
	/** State of the flight stick from the previous frame */
	GameInputFlightStickState PreviousState = {};

	// Repeat times for any pressed buttons
	static constexpr uint32 MaxSupportedButtons = 32;
	double RepeatTime[MaxSupportedButtons];

	/** Repeat times for when switches are pressed (aka the DPad) */
	TArray<double> SwitchRepeatTimes;
};

#endif	// GAME_INPUT_SUPPORT