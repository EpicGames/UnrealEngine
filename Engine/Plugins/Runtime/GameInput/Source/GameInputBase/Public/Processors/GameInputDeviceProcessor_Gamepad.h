// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

/**
* Processor for Gamepad Inputs
*/
class FGameInputGamepadDeviceProcessor : public IGameInputDeviceProcessor
{
protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	GAMEINPUTBASE_API bool ProcessGamepadAnalogState(const FGameInputEventParams& Params, GameInputGamepadState& GamepadState);
	GAMEINPUTBASE_API virtual bool ProcessGamepadButtonState(const FGameInputEventParams& Params, GameInputGamepadState& GamepadState);

	GameInputGamepadState PreviousState = {};

	uint32 LastButtonHeldMask = 0;

	uint32 LastAnalogButtonHeldMask = 0;

	enum { MaxSupportedButtons = 32 };

	double RepeatTime[MaxSupportedButtons] = { };

	/** 
	* Keeps track of how many times this gamepad has been processed this frame. 
	* Every successful processing of button input in ProcessInput will increment this value.
	* This will get reset at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;
};

#endif	// GAME_INPUT_SUPPORT