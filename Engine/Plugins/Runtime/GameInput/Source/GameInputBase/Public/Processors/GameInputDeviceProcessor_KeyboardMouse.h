// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

/**
* Processor for Keyboard inputs
*/
class FGameInputKeyboardDeviceProcessor : public IGameInputDeviceProcessor
{
protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	GAMEINPUTBASE_API virtual void UpdateUnifiedKeyboardState(const FGameInputEventParams& Params, TSet<uint8>& CurrentPressedKeys);
	
	GAMEINPUTBASE_API bool IsKeyPressed(uint8 VirtualKeyCode) const;

	GAMEINPUTBASE_API virtual void SetSimulatedCapsLock(bool bVal);

	TSet<uint8> LastPressedKeys;

	TMap<uint8, double> KeyRepeatTime;	

	// cannot read actual caps lock state, so necessary to simulate it as a bool toggle
	bool bSimulatedCapsLock = {};
};


/**
* Processor for mouse inputs
*/
class FGameInputMouseDeviceProcessor : public IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API explicit FGameInputMouseDeviceProcessor(const TSharedPtr<class ICursor>& InCursor);

protected:	
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	GAMEINPUTBASE_API virtual bool CanProcessVirtualMouse() const;

	TSharedPtr<class ICursor> Cursor = nullptr;

	GameInputMouseState PreviousMouseState = {};

	FVector2D LastMouseOffset = {};

	enum { MaxSupportedButtons = 32 };
	double RepeatTime[MaxSupportedButtons];

	uint32 LastButtonHeldMask = 0;
};

#endif	// GAME_INPUT_SUPPORT