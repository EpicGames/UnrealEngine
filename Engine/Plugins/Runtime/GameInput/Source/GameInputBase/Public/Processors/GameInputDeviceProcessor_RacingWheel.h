// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

/**
* Processor for racing wheel types (GameInputKindRacingWheel)
* 
* Note: Racing wheels are often paired with other device processors like Gamepad/Controller
* to handle any "normal" buttons that may be on them.
*/
class FGameInputRacingWheelProcessor : public IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API FGameInputRacingWheelProcessor();
protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	/**
	* Returns the deadzone that we would like to use for racing wheel analog inputs.
	*/
	static GAMEINPUTBASE_API const float GetRacingWheelDeadzone();

	/**
	* Processes the analog inputs of the wheel state (brake/clutch pedals, wheel movement, etc).
	*/
	GAMEINPUTBASE_API bool ProcessWheelAnalogState(const FGameInputEventParams& Params, GameInputRacingWheelState& CurrentWheelState);

	/**
	* Processes any buttons on the racing wheel (next/previous gears, menu nav, etc);
	*/
	GAMEINPUTBASE_API bool ProcessWheelButtonState(const FGameInputEventParams& Params, GameInputRacingWheelState& CurrentWheelState);

	/**
	* Keeps track of how many times this gamepad has been processed this frame.
	* Every successful processing of button input in ProcessInput will increment this value.
	* This will get reset at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;

	/** The previous game input state that was last processed. */
	GameInputRacingWheelState PreviousState;

	/** 
	* Array of repeat times to calculate if a button has been held long enough
	* to receive an IE_REPEAT event.
	*/
	static const uint32 MaxSupportedButtons = 16;
	double RepeatTime[MaxSupportedButtons];
};


#endif	// GAME_INPUT_SUPPORT