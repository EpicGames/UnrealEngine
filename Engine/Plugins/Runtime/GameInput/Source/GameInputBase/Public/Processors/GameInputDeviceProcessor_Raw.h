// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

struct FGameInputDeviceConfiguration;
struct FGameInputRawDeviceReportData;

#if UE_GAMEINPUT_SUPPORTS_RAW

/**
* Processor for raw input types (GameInputKindRawDeviceReport)
* 
* Raw input is very customizable and is very different per-input device, so it requires a lot of configuration.
* Most of the time it is special analog inputs on custom third party devices. A good example is the Whammy bar on
* a guitar controller, or maybe some special shifter on a racing wheel that the Game Input API does not pick up by default.
*/
class FGameInputRawDeviceProcessor : public IGameInputDeviceProcessor
{
protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	/**
	* Reads the current state of the raw input buffer and populates CurrentRawData and PreviousRawData
	*/
	UE_DEPRECATED(5.9, "Use ReadCurrentRawInputStateInfo instead.")
	GAMEINPUTBASE_API const GameInputRawDeviceReportInfo* ReadCurrentRawInputState(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse);

private:
	
	/**
	 * Reads the raw report info for GameInput API version 0. If the Api version is different then 0, this will return nullptr.	 
	 */
	const GameInputRawDeviceReportInfo* Internal_AttemptToReadCurrentRawInputState(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse);

protected:
	/**
	 * Returns the raw report info about the given reading which can be used to access the raw serialized data such as the size, report id, and more.	 
	 */
	GAMEINPUTBASE_API GameInputRawDeviceReportInfo ReadCurrentRawInputStateInfo(const FGameInputEventParams& Params, IGameInputReading* ReadingToUse);

	/**
	* Translates the given uint8 RawValue to a float between 0.0 and +1.0, like most gamepad analog "triggers"
	* 
	* @param RawValue
	* 
	* @return			A float between 0.0 and +1.0
	*/
	GAMEINPUTBASE_API constexpr float RawValueToFloatTrigger(const uint8 RawValue) const;

	/**
	 * Converts the lower 4 bits of the given raw value to a value between 0.0f and +1.0f based on the given scale.
	 */
	GAMEINPUTBASE_API constexpr float RawValueNibbleValueToFloat(const uint8 RawValue, const float Scale = 15.0f) const;

	/**
	* Translates the given uint8 RawValue to a float between -1.0 and +1.0.
	* 
	* @param RawValue
	* @param Deadzone
	* 
	* @return A float between -1.0 and +1.0. Will be 0.0 if the value is within the deadzone.
	*/
	GAMEINPUTBASE_API const float RawValueToFloatAnalog(uint8 RawValue, const uint8 DeadZone = 2) const;

	/**
	* Processes all the current raw input data. This should only be called after we have read our current raw input report
	* or we have manually set the values (i.e. when we clear the input)
	*/
	GAMEINPUTBASE_API bool ProcessAllRawValues(const FGameInputEventParams& Params, const FGameInputDeviceConfiguration* DeviceConfig, const bool bShouldProcessButtons, const bool bShouldProcessAnalog);

	/**
	* Processes the raw value at the given index as a button bitmask. 
	* For every bit in the raw value and treat it as 0 for pressed and 1 for on. 
	*/
	GAMEINPUTBASE_API bool ProcessRawInputValueAsBitmask(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData);

	/**
	* Processes the raw value at the given index as a button value. If the value is non-zero, then treat it as pressed. 
	* If the value is zero, treat it as unpressed.
	*/
	GAMEINPUTBASE_API bool ProcessRawInputValueAsButton(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData);

	/**
	* Processes the raw value at the given index as an analog value. This will translate the uint8 to a float value 
	* and call OnControllerAnalog for it.
	*/
	GAMEINPUTBASE_API bool ProcessRawInputValueAsAanalog(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData);

	/**
	* Processes a pair of raw input indexes. This allows you to combine the values of multiple uint8 axis readings and combine them
	*/
	GAMEINPUTBASE_API bool ProcessRawInputValueAsAanalogPaired(const FGameInputEventParams& Params, const FGameInputRawDeviceReportData* AxisData);

	GAMEINPUTBASE_API bool ProcessRawInputValuePackedUInt8(const FGameInputEventParams& Params, const int32 RawValueIndex, const FGameInputRawDeviceReportData* AxisData);

	void EvaluatePackedUInt8ButtonChordsAsAnalog(
			const FGameInputEventParams& Params,
			const uint8 CurrentValue,
			const uint8 PreviousValue,
			const TArray<FGamepadKeyNames::Type>& ButtonNames);

	void EvaluatePackedUInt8ButtonChordsAsButtonEvents(
			const FGameInputEventParams& Params,
			const uint8 CurrentValue,
			const uint8 PreviousValue,
			double* RepeatTime,
			const TArray<FGamepadKeyNames::Type>& ButtonNames);

	/**
	* The current raw input data that is populated by IGameInputRawDeviceReport::GetRawData
	*/
	TArray<uint8> CurrentRawData;

	/**
	* The previous frame's raw input data that is set at the end of ProcessInput.
	*/
	TArray<uint8> PreviousRawData;

	/** A struct that contains info about each raw input index */
	struct FPerRawInputIndexData
	{
		// The max number of buttons possibly associated with this raw input data. The raw input data is
		// a uint8, so we can have at maximum a button per-bit, so 8.
		enum { MaxSupportedButtons = 8 };

		// Stores the times that each button index has been pressed so we can detect key repeats
		double RepeatTime[MaxSupportedButtons];

		/** 
		* A map of key names to int values that is generated dynamically based on the button mapping.
		*/
		TMap<uint32, FName> KeyNameMap;
	};

	/**
	* A map of an index to some raw data associated with it. 
	* This is only used for button evaluation, not analog.
	*/
	TMap<int32, FPerRawInputIndexData> RawInputIndexDataMap;

	/**
	* Keeps track of how many times this gamepad has been processed this frame.
	* Every successful processing of button input in ProcessInput will increment this value.
	* This will get reset at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;
};

#endif	// UE_GAMEINPUT_SUPPORTS_RAW

#endif	// GAME_INPUT_SUPPORT