// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

#if UE_GAMEINPUT_SUPPORTS_SENSORS
/**
* Processor for Gamepad "Sensor" Inputs such as gyro or accelerometers
*/
UE_EXPERIMENTAL(5.8, "Experimental sensor data, API may change in the future")
class FGameInputGamepadSensorDeviceProcessor final : public IGameInputDeviceProcessor
{
protected:
	virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	virtual bool PostProcessInput(const FGameInputEventParams& Params) override;
	virtual void ClearState(const FGameInputEventParams& Params) override;
	virtual GameInputKind GetSupportedReadingKind() const override;
	
	/**
	 * Process the current sensor state and report it to the engine via the message handler.
	 * 
	 * @param Params Current input param data
	 * @param CurrentState The current sensor state for this frame
	 * @return True if there was any sensor data reported.
	 */
	bool ProcessGamepadSensorState(const FGameInputEventParams& Params, GameInputSensorsState& CurrentState);
	/**
	 * The sensor data from the previous reading that was processed
	 */
	GameInputSensorsState PreviousState = {};

	/**
	 * Accumulated sensor state for passing on once all input has been processed.
	 */
	GameInputSensorsState AccumulatedState = {};

	/**
	* Keeps track of how many times this gamepad has been processed this frame.
	* Every successful processing of sensor input in ProcessInput will increment this value.
	* This will get set to 0 or a negative value at the end of the input frame in PostProcessInput.
	*/
	int32 NumReadingsProcessedThisFrame = 0;
};
#endif// #if UE_GAMEINPUT_SUPPORTS_SENSORS

#endif	// GAME_INPUT_SUPPORT