// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/IGameInputDeviceProcessor.h"

#if GAME_INPUT_SUPPORT

#if UE_GAMEINPUT_SUPPORTS_TOUCH

/**
* Processor for touch inputs
*/
class FGameInputTouchDeviceProcessor : public IGameInputDeviceProcessor
{
protected:
	GAMEINPUTBASE_API virtual bool ProcessInput(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual void ClearState(const FGameInputEventParams& Params) override;
	GAMEINPUTBASE_API virtual GameInputKind GetSupportedReadingKind() const override;

	struct FTouchData
	{
		FTouchData()
		{
			TouchId = INDEX_NONE;
			Position = FVector2D::ZeroVector;
			Pressure = 0.0f;
			bHasMoved = false;
			bIsActive = false;
		}

		int64 TouchId;
		FVector2D Position;
		float Pressure;
		bool bHasMoved;
		bool bIsActive;
	};
	TArray<FTouchData> PreviousTouchData;
	int32 ActiveTouchPoints = 0;

	// The max touch X and Y of the touch region to let us scale input based on the size of the touch area
	int32 MaxTouchX = 1920;
	int32 MaxTouchY = 1080;
};
#endif	// #if UE_GAMEINPUT_SUPPORTS_TOUCH


#endif	// GAME_INPUT_SUPPORT