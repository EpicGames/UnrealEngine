// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/CoreMiscDefines.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GameInputBaseIncludes.h"

#if GAME_INPUT_SUPPORT

/**
* A Game Input Device Processor is used to determine the state of a device
* each frame and send messages to the given Message Handler when we want to. 
* Each processor represents a single "kind" of Game Input device. 
* 
* They are created by the owning GameInput Device Container (GameInputDeviceContainer.h)
* and polled each frame when possible. Each Human Interface Device can be made up of multiple 
* Game Input Kinds, so a single device container may have multiple different processors associated with it.
*/
class IGameInputDeviceProcessor
{
public:
	GAMEINPUTBASE_API IGameInputDeviceProcessor();

	virtual ~IGameInputDeviceProcessor() = default;

	struct FGameInputEventParams
	{
		/** The current reading for this input event */
		IGameInputReading* Reading = nullptr;

		/** The previous reading from the last frame. */
		IGameInputReading* PreviousReading = nullptr;

		/** Device associated with this reading */
		IGameInputDevice* Device = nullptr;

		/** The message handler to use to send any input events */
		TSharedPtr<FGenericApplicationMessageHandler> MessageHandler = nullptr;

		/** The platform user that is associated with the input device that triggered this event */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;

		/** The Input device ID of the device that triggered this event. */
		FInputDeviceId InputDeviceId = INPUTDEVICEID_NONE;

		/** Gets the Input Device Info for this GameInput Device if it is valid. Can be null. */
		GAMEINPUTBASE_API const GameInputDeviceInfo* GetDeviceInfo() const;
	};

	/** 
	* Process any input events from the given reading and send events to the message handler
	* using the given Platform User and Input Device Id's.
	* 
	* This can be called multiple times per frame, like if we are running at a lower FPS.
	* This is because GameInput may accumulate multiple readings to use by the time the game thread gets here.
	* 
	* If you have inputs that you only want to process once per frame, then use PostProcessInput instead.
	* 
	* @return True if any input events have been processed, false if nothing happened
	*/
	virtual bool ProcessInput(const FGameInputEventParams& Params) = 0;

	/**
	* Called after all current Game Input readings have been processed
	* this frame and there are no more readings for a device (GetNextReading returns GAMEINPUT_E_READING_NOT_FOUND). 
	* This is called regardless of if an accompanying ProcessInput call has happened.
	* 
	* This function is useful for dealing with multiple readings in a single frame for analog devices. 
	* It is recommended that processors override this function and only process analog input events here at
	* the end of the frame, once, instead of multiple times in ProcessInput. That is because at lower frame rates
	* or hitches you will end up with more then one GameInput reading in the stack that we need to process. If you were to
	* process all of those analog inputs in a single frame, the values would accumulate to above or below +-1.0,
	* which could have unexpected behavior in Slate.
	* 
	* @param Params		Game Input event params sent from the owning FGameInputDeviceContainer.
	*					Params.Reading should always be null here, as there is no current reading.
	*					Params.PreviousReading will contain the most recent Game Input reading from the input stack.
	* 
	* @return			True if any input events have been processed, false if nothing happened
	*/
	GAMEINPUTBASE_API virtual bool PostProcessInput(const FGameInputEventParams& Params);

	/** 
	* Clear any input state related to this processor, typically by sending events to the 
	* message handler that each FKey related to this processor now has a value of 0.
	*/
	virtual void ClearState(const FGameInputEventParams& Params) = 0;

	/*
	* Returns the kind of reading that this processor supports. If the current GameInput reading 
	* is not included in this bitmask, then this processor will not have it's "ProcessInput" 
	* function called.
	* 
	* @see FGameInputDeviceContainer::ProcessInput
	*/
	virtual GameInputKind GetSupportedReadingKind() const = 0;

protected:	

	/** 
	* Returns the string hardware device identifier for the given input event params that can be used
	* to create an FInputDeviceScope
	*/
	UE_DEPRECATED(5.8, "Use UE::GameInput::GetHardwareDeviceIdentifierName instead")
	GAMEINPUTBASE_API const FString GetHardwareDeviceIdentifierName(const IGameInputDeviceProcessor::FGameInputEventParams& Params) const;

	/** A general use function to call the message handler and tell it about a controller analog key being used */
	GAMEINPUTBASE_API void OnControllerAnalog(const FGameInputEventParams& Params, const FName& GamePadKey, float NewAxisValueNormalized, float OldAxisValueNormalized, float DeadZone, const bool bSetDeviceScope = true);

	/**
	 * Helper function for processing the button states of Game input.
	 *
	 * @param CurrentButtonHeldMask			The current state of the button mask to evaluate
	 * @param PreviousButtonMask		The previous state of the button mask. This value will be modified to store the current button mask after evaluation
	 * @param RepeatTime				A map of the button index to a time that it was last active, used to determine if the key meets the qualifications for a repeat event
	 * @param UnrealButtonNameMap		A map of Unreal Engine Gamepad key names to their uint32 mapping from Game Input.
	 * @param SupportedButtonCount		The maximum number of buttons we can process
	 */
	GAMEINPUTBASE_API void EvaluateButtonStates(
		const FGameInputEventParams& Params,
		const uint32 CurrentButtonHeldMask,
		uint32& PreviousButtonMask,
		double* RepeatTime,
		const TMap<uint32, FGamepadKeyNames::Type>& UnrealButtonNameMap,
		const uint32 SupportedButtonCount);

	/**
	* Helper function to processing the state of a Switch position (aka a DPad, like left/right/up/down)
	* 
	* @param Params		The GameInput event params
	* @param CurrentPosition	The current position of the switch
	* @param PreviousPosition	The previous position of the switch. This will be set to the value of the current switch at the end of this function
	* @param RepeatTimes		Array of doubles that represent the time at which a switch was last pressed that we can use to check for when to send repeat events
	*/
	GAMEINPUTBASE_API void EvaluateSwitchState(
		const FGameInputEventParams& Params,
		GameInputSwitchPosition CurrentPosition,
		GameInputSwitchPosition& PreviousPosition,
		TArray<double>& RepeatTimes);

	/** Delay before sending a repeat message after a button was first pressed */
	float InitialButtonRepeatDelay;

	/** Delay before sending a repeat message after a button has been pressed for a while */
	float ButtonRepeatDelay;
};

#endif	// GAME_INPUT_SUPPORT
