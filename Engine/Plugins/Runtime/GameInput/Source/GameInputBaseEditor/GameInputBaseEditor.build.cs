// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameInputBaseEditor : ModuleRules
{
	public GameInputBaseEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"GameInputBase",
				"Settings",
				"UnrealEd"
			}
		);
	}
}
