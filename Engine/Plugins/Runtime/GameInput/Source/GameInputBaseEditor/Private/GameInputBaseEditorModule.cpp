// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputUtils.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "ISettingsModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GameInputDeveloperSettings.h"
#include "GameInputLogging.h"
#include "UnrealEdMisc.h"
#include "Dialogs/Dialogs.h"

#if GAME_INPUT_SUPPORT
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
	#include <shellapi.h>
#endif

THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif //#if GAME_INPUT_SUPPORT

#define LOCTEXT_NAMESPACE "GameInputBaseEditor"

namespace UE::Input
{
	static bool bEnableGameInputVersionCheck = true;
	static FAutoConsoleVariableRef CVarEnableGameInputVersionCheck(
		TEXT("UE.GameInput.bEnableGameInputVersionCheck"),
		bEnableGameInputVersionCheck,
		TEXT("If true, then we will check for a valid GameInput version when booting the editor and prompt users to install the GameInputRedist.msi if necessary"),
		ECVF_Default);
}

/**
* Editor module for Game Input that will register the UGameInputDeveloperSettings so they
* show up as editable project settings in the editor.
*/
class FGameInputBaseEditorModule final : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
		// register settings
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "Input",
				LOCTEXT("ControllerSettingsName", "Game Input Device Settings"),
				LOCTEXT("ControllerSettingsDescription", "Settings for the Game Input plugin.\n\n"
								   "\"Game Input\" is an API that interfaces with input devices of all kinds.\n"
								   "This replaces the functionality of the older XInput and RawInput interfaces in Unreal, as well as some of the WinDualShock plugin.\n"
								   "The minimum Windows version required for Game Input is Windows 10 19H1 (May 2019 update)."),
				GetMutableDefault<UGameInputDeveloperSettings>()
			);
		}
#if GAME_INPUT_SUPPORT
		// Listen for when the editor is ready until we try and pop up a nice and friendly widget to upgrade
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().AddRaw(this, &FGameInputBaseEditorModule::OnMainFrameCreationFinished);
#endif
	}

	virtual void ShutdownModule() override
	{	
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "Input");
		}

#if GAME_INPUT_SUPPORT
		// Remove any listeners to the editor startup delegate
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule::Get().OnMainFrameCreationFinished().RemoveAll(this);
		}
#endif
	}

#if GAME_INPUT_SUPPORT

	bool ShouldValidateGameInput()
	{
		// let users cvar this off
		if (!UE::Input::bEnableGameInputVersionCheck)
		{
			return false;
		}
		
		// We don't care for Game Input if we are running a commandlet, like when we are cooking.
		if (IsRunningCommandlet())
		{
			return false;
		}

		// If there is no project name then we don't need game input either. This means we are in the project launcher
		if (!FApp::HasProjectName())
		{
			return false;
		}

		// Unattended app can't receive any user input, so there is no need to try and create the GameInput interface.
		if (FApp::IsUnattended() && !FApp::AllowUnattendedInput())
		{
			UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because it is unattended (FApp::IsUnattended is true) and thus cannot receive user input. GameInput will not be initialized.");
			return false;
		}

		// Doesn't make sense to have headless apps create game input
		if (!FApp::CanEverRender())
		{
			UE_LOGF(LogGameInput, Log, "GameInputBase module is exiting because it cannot render anything (FApp::CanEverRender is false). GameInput will not be initialized.");
			return false;
		}

		return true;
	}
	
	void OnMainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog)
	{
		if (ShouldValidateGameInput())
		{
			ValidateGameInputAvailability();
		}	
	}

	/**
	 * Check if the user currently has the min version of GameInput installed.
	 * 
	 * If they don't, then prompt them to install it now
	 */
	void ValidateGameInputAvailability()
	{
		if (!UE::Input::bEnableGameInputVersionCheck)
		{
			return;
		}
		
		if (UE::GameInput::HasValidVersionOfGameInput())
		{			
			UE_LOGF(LogGameInput, Log, "A valid Game Input installation was detected: %ls", *UE::GameInput::GetInstalledGameInputVersionInfoString());
			return;
		}

		RunGameInputInstallerUserFlow();
	}

	bool PromptUserToInstallGameInput()
	{
		const FString GameInputInstallerPath = GetFullGameInputInstallerPath();

		FFormatNamedArguments Args;
		Args.Add(TEXT("InstallerPath"), FText::FromString(GameInputInstallerPath));
		Args.Add(TEXT("DetectedVersionNum"), FText::FromString(UE::GameInput::GetInstalledGameInputVersionInfoString()));
		Args.Add(TEXT("MinVersionNum"), FText::FromString(UE::GameInput::GetMinVersionOfGameInputRedistString()));
		
		// Pop up a widget with a yes/no prompt to run the Game Input installer
		const FText RequiresInstallerMessageText =
			FText::Format(LOCTEXT("GameInput_RequiresInstall_Prompt",
			"A valid version of Game Input was not detected ({DetectedVersionNum}). Some input devices may not work as expected.\n\n"
			"Would you like to run the Game Input {MinVersionNum} installer now? An editor reboot will be required.\n\n"
			"{InstallerPath}"),
			Args);
		
		const FText Title = LOCTEXT("GameInput_RequiresInstall_Prompt_Title", "Missing Game Input Installation");

		FSuppressableWarningDialog::FSetupInfo Info (RequiresInstallerMessageText, Title, TEXT("ShowGameInputRedistInstallPrompt"), GEditorPerProjectIni);
		Info.ConfirmText = LOCTEXT("RunGameInputInstallerNow", "Install Now");
		Info.CancelText = LOCTEXT("RunGameInputInstallerNow_Later", "No");
		
		FSuppressableWarningDialog RequireGameInputInstallPrompt(Info);
		FSuppressableWarningDialog::EResult RunInstallerRes = RequireGameInputInstallPrompt.ShowModal();
		
		if (RunInstallerRes == FSuppressableWarningDialog::EResult::Cancel ||
			RunInstallerRes == FSuppressableWarningDialog::EResult::Suppressed)
		{
			return false;
		}
		
		return true;
	}

	static FString GetFullGameInputInstallerPath()
	{
		return FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + TEXT("Extras/Redist/en-us/GameInputRedist.msi"));
	}

	/**
	 * Actually executes the game input installer
	 * @return The exit code of the Game input installer.
	 */
	uint32 ExecuteGameInputInstaller()
	{
		// Note: I would have liked to use FPlatformProcess::ExecProcess here, but because that adds the additional
		// ShellExecuteInfo.lpVerb = TEXT("runas"); argument, it fails to run the installer MSI.
		// So, fall back to the actual SHELLEXECUTEINFO API
		
#if PLATFORM_WINDOWS
		const FString GameInputInstallerPath = GetFullGameInputInstallerPath();
		
		UE_LOGF(LogGameInput, Log, "Starting the game input installer: %ls", *GameInputInstallerPath);
		
		// Start the game input installer
		SHELLEXECUTEINFO ShellExecuteInfo;
		ZeroMemory(&ShellExecuteInfo, sizeof(ShellExecuteInfo));
		ShellExecuteInfo.cbSize = sizeof(ShellExecuteInfo);
		ShellExecuteInfo.fMask = SEE_MASK_NOCLOSEPROCESS;
		ShellExecuteInfo.nShow = SW_SHOWNORMAL;
		ShellExecuteInfo.lpFile = *GameInputInstallerPath;
		
		if (!ShellExecuteExW(&ShellExecuteInfo))
		{
			UE_LOGF(LogGameInput, Error, "Failed to run the game input installer: %ls", *GameInputInstallerPath);
			return 9000;
		}

		// Wait for the process to complete, then get its exit code
		WaitForSingleObject(ShellExecuteInfo.hProcess, INFINITE);

		DWORD ProcessExitCode = 0;
		GetExitCodeProcess(ShellExecuteInfo.hProcess, &ProcessExitCode);
		CloseHandle(ShellExecuteInfo.hProcess);
		
		return (uint32)ProcessExitCode;
#else
		return 0;
#endif	// #if PLATFORM_WINDOWS
	}
	
	/**
	 * Prompt the user to run the GameInputRedist.msi installer.
	 */
	void RunGameInputInstallerUserFlow()
	{
		// Ensure that we have a valid installer path before prompting users to run it
		const FString GameInputInstallerPath = GetFullGameInputInstallerPath();

		if (!IFileManager::Get().FileExists(*GameInputInstallerPath))
		{
			UE_LOGF(LogGameInput, Error, "Failed to locate the Game Input Installer: %ls", *GameInputInstallerPath);
			return;
		}
		
		// The user has elected not to install game input
		if (!PromptUserToInstallGameInput())
		{
			UE_LOGF(LogGameInput, Warning, "Game Input was not detected, and the user has denied installation. Some input devices may not work.");
			return;
		}
		
		const uint32 ExitCode = ExecuteGameInputInstaller();
		// 1638: Newer version already installed
		const bool bSuccessfulInstall = ExitCode == 0 || ExitCode == 1638;

		UE_LOGF(LogGameInput, Display, "Game Input Installer returned exit code %d", ExitCode);

		// If we failed, then notify the user that some devices may not work and give them the error code
		if (!bSuccessfulInstall)
		{
			const FText InstallFailedMessage =
				FText::FormatOrdered(LOCTEXT("GameInputInstaller_Failed_Message",
				"Game Input installation failed with error code {0}\n\nSome input devices may not behave as expected."),
				ExitCode);

			const FText InstallationSuccessMessageTitle =
				LOCTEXT("GameInputInstaller_Failed_Title",
				"Installation Failed");
			
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, InstallFailedMessage, InstallationSuccessMessageTitle);
			return;
		}

		// Otherwise the installation was a success, and in order to re-initialize Game Input we need to reboot the editor. Prompt the user to do so now
		const FText RebootEditorPrompt =
			LOCTEXT("GameInput_RebootEditor_Prompt",
			"Game Input installation was a success! For the changes to take effect, you must restart the editor.\n\nWould you like to restart now?");

		const FText InstallationSuccessMessageTitle =
			LOCTEXT("GameInput_InstallSuccess_Title",
			"Installation Successful");
		
		const EAppReturnType::Type RebootEditorRes = FMessageDialog::Open(EAppMsgCategory::Success, EAppMsgType::YesNo, RebootEditorPrompt, InstallationSuccessMessageTitle);

		UE_CLOGF(RebootEditorRes == EAppReturnType::No, LogGameInput, Warning, "Game Input was installed successfully, but without restarting the editor some input devices may not work.");
		
		if (RebootEditorRes == EAppReturnType::Yes)
		{
			constexpr bool bWarn = false;
			FUnrealEdMisc::Get().RestartEditor(bWarn);
		}
	}
#endif// #if GAME_INPUT_SUPPORT
};

IMPLEMENT_MODULE(FGameInputBaseEditorModule, GameInputBaseEditor);

#undef LOCTEXT_NAMESPACE