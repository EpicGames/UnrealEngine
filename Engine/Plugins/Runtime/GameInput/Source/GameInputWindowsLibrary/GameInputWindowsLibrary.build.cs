// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class GameInputWindowsLibrary : ModuleRules
	{
		public GameInputWindowsLibrary(ReadOnlyTargetRules Target) : base(Target)
		{
			Type = ModuleType.External;

			// This is the third party library provided by Microsoft for GameInput on Windows.
			// It comes from their NuGet repository: https://www.nuget.org/packages/Microsoft.GameInput/
			// Current version is: 3.1.26100.6879
			
			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				// Add the third party include folders so that we can include GameInput.h
				PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "ThirdParty"));

				// Add the GameInput.lib as a dependency.
				// This is located in Engine\Plugins\Runtime\GameInput\Source\GameInputWindowsLibrary\ThirdParty\Binaries\x64
				string LibPath = Path.GetFullPath(Path.Combine(ModuleDirectory, "ThirdParty/Binaries/x64"));

				PublicAdditionalLibraries.Add(Path.Combine(LibPath, "GameInput.lib"));
			}
		}
	}
}
